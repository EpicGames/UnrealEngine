// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTable.h"

#include "DSP/FloatArrayMath.h"
#include "WaveTableSampler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTable)


namespace WaveTable
{
	FWaveTableView::FWaveTableView(const TArray<float>& InSamples, const float InFinalValue)
		: SampleView(InSamples)
		, FinalValue(InFinalValue)
	{
	}

	FWaveTableView::FWaveTableView(const TArrayView<const float>& InSamples, const float InFinalValue)
		: SampleView(InSamples)
		, FinalValue(InFinalValue)
	{
	}

	int32 FWaveTableView::Num() const
	{
		return SampleView.Num();
	}

	bool FWaveTableView::IsEmpty() const
	{
		return SampleView.IsEmpty();
	}

	FWaveTable::FWaveTable(const TArray<float>& InSamples, const float InFinalValue)
		: Samples(InSamples)
		, FinalValue(InFinalValue)
	{
	}

	FWaveTable::FWaveTable(TArray<float>&& InSamples, const float InFinalValue)
		: Samples(MoveTemp(InSamples))
		, FinalValue(InFinalValue)
	{
	}

	FWaveTable::FWaveTable(const ::FWaveTableData& InData)
		: FinalValue(InData.GetFinalValue())
	{
		switch (InData.GetBitDepth())
		{
			case EWaveTableBitDepth::PCM_16:
			{
				TArrayView<const int16> Samples16Bit;
				ensureAlwaysMsgf(InData.GetDataView(Samples16Bit), TEXT("PCM_16 BitDepth format mismatch"));
				Samples.AddUninitialized(Samples16Bit.Num());
				Audio::ArrayPcm16ToFloat(Samples16Bit, Samples);
			}
			break;

			case EWaveTableBitDepth::IEEE_Float:
			{
				TArrayView<const float> SamplesFloat;
				ensureAlwaysMsgf(InData.GetDataView(SamplesFloat), TEXT("IEEE_Float BitDepth format mismatch"));
				Samples = SamplesFloat;
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
				checkNoEntry();
			}
			break;
		}
	}

	float* FWaveTable::GetData()
	{
		return Samples.GetData();
	}

	const float* FWaveTable::GetData() const
	{
		return Samples.GetData();
	}

	FWaveTableView FWaveTable::GetView() const
	{
		return FWaveTableView(Samples, FinalValue);
	}

	TArrayView<float> FWaveTable::GetSamples()
	{
		return Samples;
	}

	int32 FWaveTable::Num() const
	{
		return Samples.Num();
	}

	void FWaveTable::Set(TArray<float>&& InSamples)
	{
		Samples = MoveTemp(InSamples);
	}

	void FWaveTable::Set(TArrayView<const float> InSamples)
	{
		Samples = InSamples;
	}

	void FWaveTable::SetFinalValue(const float InValue)
	{
		FinalValue = InValue;
	}

	void FWaveTable::SetNum(int32 InSize)
	{
		Samples.SetNum(InSize);
	}

	void FWaveTable::Zero()
	{
		FMemory::Memset(Samples.GetData(), 0, sizeof(float) * Samples.Num());
	}
} // namespace WaveTable


FWaveTableData::FWaveTableData(EWaveTableBitDepth InBitDepth)
	: BitDepth(InBitDepth)
{
}

FWaveTableData::FWaveTableData(TArrayView<const float> InSamples, float InFinalValue)
	: BitDepth(EWaveTableBitDepth::IEEE_Float)
	, Data(TArrayView<uint8>((uint8*)InSamples.GetData(), InSamples.Num() * sizeof(float)))
	, FinalValue(InFinalValue)
{
}

FWaveTableData::FWaveTableData(TArrayView<const int16> InSamples, float InFinalValue)
	: BitDepth(EWaveTableBitDepth::PCM_16)
	, Data(TArrayView<uint8>((uint8*)InSamples.GetData(), InSamples.Num() * sizeof(int16)))
	, FinalValue(InFinalValue)
{
}

#if WITH_EDITOR
FName FWaveTableData::GetBitDepthPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(FWaveTableData, BitDepth);
}
#endif // WITH_EDITOR

void FWaveTableData::ArrayMixIn(TArrayView<float> OutFloatBuffer, float Gain) const
{
	checkf(OutFloatBuffer.Num() == GetNumSamples(), TEXT("Float buffer size must match that of WaveTableData"));
	switch (BitDepth)
	{
		case EWaveTableBitDepth::IEEE_Float:
		{
			TArrayView<const float> DataView;
			GetDataView(DataView);
			Audio::ArrayMixIn(DataView, OutFloatBuffer, Gain);
		}
		break;

		case EWaveTableBitDepth::PCM_16:
		{
			TArrayView<const int16> DataView;
			GetDataView(DataView);
			Audio::ArrayMixIn(DataView, OutFloatBuffer, Gain);
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
			checkNoEntry();
		}
		break;
	}
}

int32 FWaveTableData::GetSampleSize() const
{
	switch (BitDepth)
	{

		case EWaveTableBitDepth::PCM_16:
		{
			return sizeof(int16);
		}

		case EWaveTableBitDepth::IEEE_Float:
		{
			return sizeof(float);
		}

		default:
		{
			static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
			checkNoEntry();
			return sizeof(uint8);
		}
	}
}

void FWaveTableData::Empty()
{
	Data.Empty();
	FinalValue = ::WaveTable::InvalidWaveTableValue;
}

bool FWaveTableData::Resample(int32 InCurrentSampleRate, int32 InNewSampleRate, ::WaveTable::FWaveTableSampler::EInterpolationMode InInterpMode)
{
	if (InCurrentSampleRate != InNewSampleRate)
	{

		const EWaveTableBitDepth LastBitDepth = BitDepth;
		FWaveTableData ResampledData(EWaveTableBitDepth::IEEE_Float);

		// 1. Init scratch WaveTableData for resampled data (must be in IEEE_Float)
		{
			ResampledData.Zero(GetNumSamples() * InCurrentSampleRate / InNewSampleRate);
		}

		// 2. Resample in correct bit depth
		{
			TArrayView<float> ResampledView;
			ResampledData.GetDataView(ResampledView);
			::WaveTable::FWaveTableSampler::FSettings Settings;
			Settings.InterpolationMode = InInterpMode;
			::WaveTable::FWaveTableSampler Sampler(MoveTemp(Settings));
			Sampler.Process(*this, ResampledView);
		}

		// 3. Move data & set to correct bit depth (ignored if original bit depth was already IEEE_Float)
		{
			*this = MoveTemp(ResampledData);
			SetBitDepth(LastBitDepth);
		}

		return true;
	}

	return false;
}

void FWaveTableData::Reset(int32 NumSamples)
{
	Data.Reset(NumSamples * GetSampleSize());
	FinalValue = ::WaveTable::InvalidWaveTableValue;
}

bool FWaveTableData::SetBitDepth(EWaveTableBitDepth InNewBitDepth)
{
	if (InNewBitDepth == BitDepth)
	{
		return false;
	}

	const int32 NumSamples = GetNumSamples();
	TArray<uint8> ConvertedData;
	switch(InNewBitDepth)
	{
		case EWaveTableBitDepth::IEEE_Float:
		{
			ConvertedData.SetNumUninitialized(sizeof(float) * NumSamples);

			TArrayView<const int16> InputWaveTable;
			GetDataView(InputWaveTable);
			Audio::ArrayPcm16ToFloat(InputWaveTable, TArrayView<float>((float*)ConvertedData.GetData(), ConvertedData.Num()));
		}
		break;

		case EWaveTableBitDepth::PCM_16:
		{
			ConvertedData.SetNumUninitialized(sizeof(int16) * NumSamples);

			TArrayView<const float> InputWaveTable;
			GetDataView(InputWaveTable);
			Audio::ArrayFloatToPcm16(InputWaveTable, TArrayView<int16>((int16*)ConvertedData.GetData(), ConvertedData.Num()));
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
			checkNoEntry();
		}
	}

	BitDepth = InNewBitDepth;
	Data = MoveTemp(ConvertedData);
	return true;
}

void FWaveTableData::SetData(TArrayView<const int16> InData, bool bIsLoop)
{
	Data = TArrayView<uint8>((uint8*)InData.GetData(), InData.Num() * sizeof(int16));
	BitDepth = EWaveTableBitDepth::PCM_16;
	if (InData.IsEmpty())
	{
		FinalValue = 0.0f;
	}
	else
	{
		constexpr float ConversionValue = 1.0f / static_cast<float>(TNumericLimits<int16>::Max());
		if (bIsLoop)
		{
			FinalValue = InData[0] * ConversionValue;
		}
		else
		{
			FinalValue = InData.Last() * ConversionValue;
		}
	}
}

void FWaveTableData::SetData(TArrayView<const float> InData, bool bIsLoop)
{
	Data = TArrayView<uint8>((uint8*)InData.GetData(), InData.Num() * sizeof(float));
	BitDepth = EWaveTableBitDepth::IEEE_Float;
	if (InData.IsEmpty())
	{
		FinalValue = 0.0f;
	}
	else
	{
		if (bIsLoop)
		{
			FinalValue = InData[0];
		}
		else
		{
			FinalValue = InData.Last();
		}
	}
}

void FWaveTableData::SetFinalValue(float InFinalValue)
{
	FinalValue = InFinalValue;
}

void FWaveTableData::SetRawData(EWaveTableBitDepth InBitDepth, TArrayView<const uint8> InData)
{
	Data = InData;
	BitDepth = InBitDepth;
}

void FWaveTableData::SetRawData(EWaveTableBitDepth InBitDepth, TArray<uint8>&& InData)
{
	BitDepth = InBitDepth;
	Data = MoveTemp(InData);
}

EWaveTableBitDepth FWaveTableData::GetBitDepth() const
{
	return BitDepth;
}

bool FWaveTableData::GetDataView(TArrayView<const float>& OutData) const
{
	if (BitDepth == EWaveTableBitDepth::IEEE_Float)
	{
		OutData = TArrayView<const float>((const float*)(Data.GetData()), Data.Num() / sizeof(float));
		return true;
	}

	return false;
}

bool FWaveTableData::GetDataView(TArrayView<float>& OutData)
{
	if (BitDepth == EWaveTableBitDepth::IEEE_Float)
	{
		OutData = TArrayView<float>((float*)(Data.GetData()), Data.Num() / sizeof(float));
		return true;
	}

	return false;
}

bool FWaveTableData::GetDataView(TArrayView<const int16>& OutData) const
{
	if (BitDepth == EWaveTableBitDepth::PCM_16)
	{
		OutData = TArrayView<const int16>((const int16*)(Data.GetData()), Data.Num() / sizeof(int16));
		return true;
	}

	return false;
}

bool FWaveTableData::GetDataView(TArrayView<int16>& OutData)
{
	if (BitDepth == EWaveTableBitDepth::PCM_16)
	{
		OutData = TArrayView<int16>((int16*)(Data.GetData()), Data.Num() / sizeof(int16));
		return true;
	}

	return false;
}

float FWaveTableData::GetLastValue() const
{
	if (Data.IsEmpty())
	{
		return 0.0f;
	}

	switch (BitDepth)
	{
		case EWaveTableBitDepth::IEEE_Float:
		{
			TArrayView<const float> TableView;
			ensureAlways(GetDataView(TableView));
			return TableView.Last();
		}

		case EWaveTableBitDepth::PCM_16:
		{
			TArrayView<const int16> TableView;
			ensureAlways(GetDataView(TableView));
			constexpr float ConversionValue = 1.0f / static_cast<float>(TNumericLimits<int16>::Max());
			return TableView.Last() * ConversionValue;
		}

		default:
		{
			static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
			checkNoEntry();
		}
		break;
	}

	return 0.0f;
}

float FWaveTableData::GetFinalValue() const
{
	if (FMath::IsNearlyEqual(FinalValue, WaveTable::InvalidWaveTableValue))
	{
		return GetLastValue();
	}

	return FinalValue;
}

int32 FWaveTableData::GetNumSamples() const
{
	return Data.Num() / GetSampleSize();
}

const TArray<uint8>& FWaveTableData::GetRawData() const
{
	return Data;
}

bool FWaveTableData::IsEmpty() const
{
	return Data.IsEmpty();
}

void FWaveTableData::Zero(int32 InNumSamples)
{
	if (InNumSamples < 0)
	{
		Data.Shrink();
		FMemory::Memzero(Data.GetData(), sizeof(uint8) * Data.Num());
	}
	else
	{
		const int32 NumBytes = InNumSamples * GetSampleSize();
		Data.Init(0, NumBytes);
	}
}
