// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveTableBank.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTableBank)


namespace WaveTable
{
	namespace BankPrivate
	{
#if WITH_EDITOR
		int32 GetEntryMaxNumSamples(const UWaveTableBank& InBank)
		{
			int32 MaxSamples = 0;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			for (const FWaveTableBankEntry& Entry : InBank.Entries)
			{
				int32 NumSamples = 0;
				if (Entry.Transform.Curve == EWaveTableCurve::File)
				{
					int32 Offset = 0;
					Entry.Transform.WaveTableSettings.GetEditSourceBounds(Offset, NumSamples);
				}
				else
				{
					NumSamples = 1 << static_cast<int32>(EWaveTableResolution::Res_Max);
				}
				MaxSamples = FMath::Max(MaxSamples, NumSamples);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			return MaxSamples;
		}

		int32 GetWaveTableNumSamples(const UWaveTableBank& InBank, const FWaveTableBankEntry& InEntry)
		{
			const FWaveTableTransform& Transform = InEntry.Transform;
			switch (InBank.SampleMode)
			{
				case EWaveTableSamplingMode::FixedSampleRate:
				{
					if (Transform.Curve == EWaveTableCurve::File)
					{
						int32 NumSourceSamples = 0;
						int32 TopOffset = 0;
						Transform.WaveTableSettings.GetEditSourceBounds(TopOffset, NumSourceSamples);
						const float ResampleFactor = InBank.SampleRate / static_cast<float>(Transform.WaveTableSettings.SourceSampleRate);
						return (int32)(NumSourceSamples * ResampleFactor);
					}
					else
					{
						return (int32)(InBank.SampleRate * Transform.GetDuration());
					}
				}

				case EWaveTableSamplingMode::FixedResolution:
				{
					const int32 MaxNumSamples = GetEntryMaxNumSamples(InBank);
					switch (InBank.Resolution)
					{
						case EWaveTableResolution::Maximum:
						{
							return FMath::Max(MaxNumSamples, 1 << static_cast<int32>(EWaveTableResolution::Res_Max));
						}

						case EWaveTableResolution::None:
						{
							EWaveTableResolution CurveRes = EWaveTableResolution::None;
							switch (Transform.Curve)
							{
								case EWaveTableCurve::File:
								{
									CurveRes = EWaveTableResolution::Maximum;
								}
								break;

								case EWaveTableCurve::Linear:
								case EWaveTableCurve::Linear_Inv:
								{
									CurveRes = EWaveTableResolution::Res_8;
								}
								break;

								case EWaveTableCurve::Exp:
								case EWaveTableCurve::Exp_Inverse:
								case EWaveTableCurve::Log:
								{
									CurveRes = EWaveTableResolution::Res_256;
								}
								break;

								case EWaveTableCurve::Sin:
								case EWaveTableCurve::SCurve:
								case EWaveTableCurve::Sin_Full:
								{
									CurveRes = EWaveTableResolution::Res_64;
								}
								break;

								default:
								{
									CurveRes = EWaveTableResolution::Res_128;
								}
								break;
							};
							return 1 << static_cast<int32>(CurveRes);
						}

						default:
						{
							return 1 << static_cast<int32>(InBank.Resolution);
						}
					}
				}

				default:
				{
					checkNoEntry();
					static_assert(static_cast<int32>(EWaveTableSamplingMode::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
					return 0;
				}
			}
		}
#endif // WITH_EDITOR
	} // namespace BankPrivate
} // namespace WaveTable

TSharedPtr<Audio::IProxyData> UWaveTableBank::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	if (!ProxyData.IsValid())
	{
#if WITH_EDITOR
		CopyToProxyData();
#else
		MoveToProxyData();
#endif // WITH_EDITOR
	}

	check(ProxyData.IsValid());
	return ProxyData;
}

void UWaveTableBank::CopyToProxyData()
{
	checkf(!ProxyData.IsValid(), TEXT("Cannot overwrite pre-existing proxy when calling UWaveTableBank::CopyToProxyData"));

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ProxyData = MakeShared<FWaveTableBankAssetProxy, ESPMode::ThreadSafe>(GetUniqueID(), SampleMode, SampleRate, GetEntries(), bBipolar);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWaveTableBank::MoveToProxyData()
{
	checkf(!ProxyData.IsValid(), TEXT("Cannot overwrite pre-existing proxy when calling UWaveTableBank::MoveToProxyData"));

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ProxyData = MakeShared<FWaveTableBankAssetProxy, ESPMode::ThreadSafe>(GetUniqueID(), SampleMode, SampleRate, MoveTemp(Entries), bBipolar);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<FWaveTableBankEntry>& UWaveTableBank::GetEntries()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Entries;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UWaveTableBank::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	FProperty* Property = InPropertyChangedEvent.Property;
	if (Property && InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// For larger banks, this is very expensive and slows the UX considerably.  This should be
		// distilled down to array entry if possible for edits on individual WaveTableSettings for
		// example to make UX more snappy and improve overall property chain edit perf.
		RefreshWaveTables();
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void UWaveTableBank::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	FProperty* Property = InPropertyChangedEvent.Property;
	if (Property && InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// For larger banks, this is very expensive and slows the UX considerably.  This should be
		// distilled down to array entry if possible for edits on individual WaveTableSettings for
		// example to make UX more snappy and improve overall property chain edit perf.
		RefreshWaveTables();
	}

	Super::PostEditChangeChainProperty(InPropertyChangedEvent);
}

void UWaveTableBank::PreSave(FObjectPreSaveContext InSaveContext)
{
	if (!InSaveContext.IsCooking())
	{
		RefreshWaveTables();
	}

	Super::PreSave(InSaveContext);
}

void UWaveTableBank::RefreshWaveTables()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WaveTableSizeMB = 0.0f;
	for (FWaveTableBankEntry& Entry : Entries)
	{
		if (Entry.Transform.Curve == EWaveTableCurve::Custom && Entry.Transform.CurveCustom.GetNumKeys() < 1)
		{
			Entry.Transform.CurveCustom.AddKey(0.0f, 0.0f);
			Entry.Transform.CurveCustom.AddKey(1.0f, 1.0f);
		}

		FWaveTableData& RuntimeData = Entry.Transform.TableData;
		const int32 NumSamples = ::WaveTable::BankPrivate::GetWaveTableNumSamples(*this, Entry);

		RuntimeData = FWaveTableData(Entry.Transform.WaveTableSettings.SourceData.GetBitDepth());
		RuntimeData.Zero(NumSamples);

		Entry.Transform.CreateWaveTable(RuntimeData, bBipolar);
		if (Entry.Transform.Curve == EWaveTableCurve::File && SampleMode == EWaveTableSamplingMode::FixedSampleRate)
		{
			Entry.Transform.Duration = NumSamples / (float)SampleRate;
		}

		WaveTableSizeMB += RuntimeData.GetRawData().Num();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	WaveTableSizeMB /= FMath::Square(1024);

	ProxyData.Reset();
}
#endif // WITH_EDITOR

void UWaveTableBank::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FWaveTableBankEntry& Entry : Entries)
		{
			Entry.Transform.VersionTableData();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
}


FWaveTableBankAssetProxy::FWaveTableBankAssetProxy(uint32 InObjectId, EWaveTableSamplingMode InSamplingMode, int32 InSampleRate, const TArray<FWaveTableBankEntry>& InBankEntries, bool bInBipolar)
	: bBipolar(bInBipolar)
	, ObjectId(InObjectId)
	, SampleRate(InSampleRate)
	, SampleMode(InSamplingMode)
{
	Algo::Transform(InBankEntries, WaveTableData, [](const FWaveTableBankEntry& Entry)
	{
		return Entry.Transform.GetTableData();
	});
}

FWaveTableBankAssetProxy::FWaveTableBankAssetProxy(uint32 InObjectId, EWaveTableSamplingMode InSamplingMode, int32 InSampleRate, TArray<FWaveTableBankEntry>&& InBankEntries, bool bInBipolar)
	: bBipolar(bInBipolar)
	, ObjectId(InObjectId)
	, SampleRate(InSampleRate)
	, SampleMode(InSamplingMode)
{
	for (FWaveTableBankEntry& Entry : InBankEntries)
	{
		WaveTableData.Add(MoveTemp(Entry.Transform.TableData));
	}
	InBankEntries.Empty();
}

FWaveTableBankAssetProxy::FWaveTableBankAssetProxy(const UWaveTableBank& InWaveTableBank)
	: bBipolar(InWaveTableBank.bBipolar)
	, ObjectId(InWaveTableBank.GetUniqueID())
	, SampleRate(InWaveTableBank.SampleRate)
	, SampleMode(InWaveTableBank.SampleMode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Algo::Transform(InWaveTableBank.Entries, WaveTableData, [](const FWaveTableBankEntry& Entry)
	{
		return Entry.Transform.GetTableData();
	});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

}
