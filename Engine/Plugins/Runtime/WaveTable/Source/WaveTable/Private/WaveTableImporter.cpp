// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableImporter.h"

#include "DSP/FloatArrayMath.h"
#include "WaveTable.h"
#include "WaveTableSettings.h"

namespace WaveTable
{
	FImporter::FImporter(const FWaveTableSettings& InSettings, bool bInBipolar)
		: Settings(InSettings)
		, bBipolar(bInBipolar)
	{
		Sampler.SetInterpolationMode(FWaveTableSampler::EInterpolationMode::Cubic);
		Sampler.SetPhase(Settings.Phase);
	}

	void FImporter::ProcessInternal(const TArrayView<const float> InEditSourceView, TArray<float>& OutWaveTable)
	{
		const FWaveTableView EditSourceView = FWaveTableView(InEditSourceView, InEditSourceView.Last());

		// 1. Resample into table
		Sampler.Reset();
		Sampler.Process(EditSourceView, OutWaveTable);

		// 2. Apply offset
		float TableOffset = 0.0f;
		if (Settings.bRemoveOffset || !bBipolar)
		{
			TableOffset = Audio::ArrayGetAverageValue(OutWaveTable);
			Audio::ArrayAddConstantInplace(OutWaveTable, -1.0f * TableOffset);
		}

		// 3. Normalize
		if (Settings.bNormalize)
		{
			const float MaxValue = Audio::ArrayMaxAbsValue(OutWaveTable);
			if (MaxValue > 0.0f)
			{
				Audio::ArrayMultiplyByConstantInPlace(OutWaveTable, 1.0f / MaxValue);
			}
		}

		// 4. Apply fades
		if (Settings.FadeIn > 0.0f)
		{
			const int32 FadeLength = RatioToIndex(Settings.FadeIn, OutWaveTable.Num());
			TArrayView<float> FadeView(OutWaveTable.GetData(), FadeLength);
			Audio::ArrayFade(FadeView, 0.0f, 1.0f);
		}

		if (Settings.FadeOut > 0.0f)
		{
			const int32 FadeLastIndex = RatioToIndex(Settings.FadeOut, OutWaveTable.Num());
			const int32 FadeInitIndex = RatioToIndex(1.0f - Settings.FadeOut, OutWaveTable.Num());
			TArrayView<float> FadeView = TArrayView<float>(&OutWaveTable[FadeInitIndex], FadeLastIndex + 1);
			Audio::ArrayFade(FadeView, 1.0f, 0.0f);
		}

		// 5. Finalize if unipolar source
		if (!bBipolar)
		{
			Audio::ArrayAbsInPlace(OutWaveTable);

			// If not requesting offset removal, add back (is always removed above for
			// unipolar to ensure fades are centered around offset)
			if (!Settings.bRemoveOffset)
			{
				Audio::ArrayAddConstantInplace(OutWaveTable, TableOffset);
			}
		}
	}

	void FImporter::Process(TArray<float>& OutWaveTable)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArrayView<const float> View = Settings.GetEditSourceView();
		if (!View.IsEmpty())
		{
			ProcessInternal(View, OutWaveTable);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FImporter::Process(::FWaveTableData& OutData)
	{
		// 1. Get editable section of source and run BDC to process light edits on float data
		int32 Offset = 0;
		int32 NumEditSamples = 0;
		Settings.GetEditSourceBounds(Offset, NumEditSamples);

		if (NumEditSamples == 0)
		{
			OutData.Zero();
			return;
		}

		// 2. Get source in float bit depth format
		TArrayView<const float> EditDataView;
		TArray<float> ScratchData;
		switch (Settings.SourceData.GetBitDepth())
		{
			case EWaveTableBitDepth::IEEE_Float:
			{
				Settings.SourceData.GetDataView(EditDataView);
				EditDataView = { EditDataView.GetData() + Offset, NumEditSamples };
			}
			break;

			case EWaveTableBitDepth::PCM_16:
			{
				TArrayView<const int16> EditDataView16Bit;
				Settings.SourceData.GetDataView(EditDataView16Bit);
				EditDataView16Bit = { EditDataView16Bit.GetData() + Offset, NumEditSamples };
				ScratchData.AddZeroed(NumEditSamples);
				Audio::ArrayPcm16ToFloat(EditDataView16Bit, ScratchData);
				EditDataView = ScratchData;
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
				checkNoEntry();
			}
			break;
		}

		// 3. Process effects
		TArray<float> ProcessedData;
		ProcessedData.AddZeroed(OutData.GetNumSamples());
		ProcessInternal(EditDataView, ProcessedData);

		// 4. Copy processed data into original buffer
		switch (OutData.GetBitDepth())
		{
			case EWaveTableBitDepth::IEEE_Float:
			{
				constexpr bool bIsLoop = true;
				OutData.SetData(ProcessedData, bIsLoop);
			}
			break;

			case EWaveTableBitDepth::PCM_16:
			{
				TArrayView<int16> TableView;
				OutData.GetDataView(TableView);
				Audio::ArrayFloatToPcm16(ProcessedData, TableView);
				if (!TableView.IsEmpty())
				{
					constexpr float ConversionValue = 1.0f / static_cast<float>(TNumericLimits<int16>::Max());
					OutData.SetFinalValue(TableView[0] * ConversionValue);
				}
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
				checkNoEntry();
			}
			break;
		}
	} // namespace Importer
} // namespace WaveTable
