// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableLogging.h"

DEFINE_LOG_CATEGORY(LogWaveTable);
