// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveTableModule.h"

#include "Modules/ModuleManager.h"

namespace WaveTable
{
	void FModule::StartupModule()
	{
	}

	void FModule::ShutdownModule()
	{
	}
} // namespace WaveTable


IMPLEMENT_MODULE(WaveTable::FModule, WaveTable);
