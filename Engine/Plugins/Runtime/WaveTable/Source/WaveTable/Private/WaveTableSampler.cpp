// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableSampler.h"

#include "Math/Vector2D.h"
#include "WaveTable.h"
#include "DSP/FloatArrayMath.h"


namespace WaveTable
{
	namespace UtilitiesPrivate
	{
		template<typename TSample>
		FVector2f GetTangentP0(const TSample* InTableView, const float NextValue, float InIndex, int32 InArraySize)
		{
			const int32 Index = FMath::TruncToInt32(InIndex);

			// Addition of array size guarantees that if index is 0, we aren't modding a negative number and returning a negative index
			const TSample LastValue = InTableView[(Index - 1 + InArraySize) % InArraySize];
			return FVector2f(2.0f, (float)(NextValue - LastValue)).GetSafeNormal();
		};

		template<typename TSample>
		FVector2f GetTangentP1(const TSample* InTableView, const float LastValue, float InIndex, int32 InArraySize)
		{
			const int32 Index = FMath::TruncToInt32(InIndex);
			const TSample NextValue = InTableView[(Index + 1) % InArraySize];
			return FVector2f(2.0f, (float)(NextValue - LastValue)).GetSafeNormal();
		};

		template<typename TSample>
		void MaxValueIndexInterpolator(TArrayView<const TSample> InTableView, TArrayView<float> InOutIndicesToValues, float InGain = 1.0f)
		{
			int32 LastIndex = 0;
			const TSample* TableData = InTableView.GetData();
			float* IndexValueData = InOutIndicesToValues.GetData();
			for (int32 i = 0; i < InOutIndicesToValues.Num(); ++i)
			{
				int32 Index = FMath::TruncToInt32(InTableView.Num() * IndexValueData[i]) % InTableView.Num();
				bool bSwapped = false;
				if (Index < LastIndex)
				{
					bSwapped = true;
					::Swap(LastIndex, Index);
				}

				TSample MaxAbs = 0.0f;
				for (int32 j = LastIndex; j <= Index; ++j)
				{
					TSample IndexAbs = FMath::Abs(TableData[j]);
					if (IndexAbs > MaxAbs)
					{
						MaxAbs = IndexAbs;
						IndexValueData[i] = InGain * (float)TableData[j];
					}
				}

				if (!bSwapped)
				{
					LastIndex = Index;
				}
			}
		};

		template <typename TSample>
		void CubicIndexInterpolator (TArrayView<const TSample> InTableView, TArrayView<float> InOutIndicesToValues, float InGain = 1.0f)
		{
			const int32 NumTableSamples = InTableView.Num();
			const TSample* InTable = InTableView.GetData();
			for (int32 i = 0; i < InOutIndicesToValues.Num(); ++i)
			{
				float& IndexToOutput = InOutIndicesToValues[i];
				const int32 LastIndexInt = FMath::TruncToInt32(IndexToOutput);

				const FVector2f P0 = { (float)LastIndexInt, (float)InTable[LastIndexInt % NumTableSamples] };
				const FVector2f P1 = { (float)(LastIndexInt + 1), (float)InTable[(LastIndexInt + 1) % NumTableSamples] };
				const FVector2f TangentP0 = GetTangentP0(InTable, P1.Y, IndexToOutput, NumTableSamples);
				const FVector2f TangentP1 = GetTangentP1(InTable, P0.Y, IndexToOutput, NumTableSamples);
				const float Alpha = IndexToOutput - LastIndexInt;

				IndexToOutput = FMath::CubicInterp(P0, TangentP0, P1, TangentP1, Alpha).Y * InGain;
			}
		};

		template<typename TSample>
		void LinearIndexInterpolator(TArrayView<const TSample> InTableView, const TArrayView<float> InOutIndicesToValues, float InGain = 1.0f)
		{
			const int32 NumTableSamples = InTableView.Num();
			const TSample* InTable = InTableView.GetData();
			for (int32 i = 0; i < InOutIndicesToValues.Num(); ++i)
			{
				float& IndexToOutput = InOutIndicesToValues[i];
				const int32 LastIndexInt = FMath::TruncToInt32(IndexToOutput);

				const float P0 = (float)InTable[LastIndexInt % NumTableSamples];
				const float Alpha = IndexToOutput - LastIndexInt;
				const float P1 = (float)InTable[(LastIndexInt + 1) % NumTableSamples];

				IndexToOutput = FMath::Lerp(P0, P1, Alpha) * InGain;
			}
		};

		template<typename TSample>
		void StepIndexInterpolator(TArrayView<const TSample> InTableView, const TArrayView<float> InOutIndicesToValues, float InGain = 1.0f)
		{
			const int32 NumTableSamples = InTableView.Num();
			const TSample* InTable = InTableView.GetData();
			float* IndexToValue = InOutIndicesToValues.GetData();
			for (int32 i = 0; i < InOutIndicesToValues.Num(); ++i)
			{
				float& IndexToOutput = IndexToValue[i];
				const int32 LastIndexInt = FMath::TruncToInt32(IndexToOutput);
				IndexToOutput = InTable[LastIndexInt % NumTableSamples] * InGain;
			}
		}
	}

	FWaveTableSampler::FWaveTableSampler()
	{
	}

	FWaveTableSampler::FWaveTableSampler(FSettings&& InSettings)
		: Settings(MoveTemp(InSettings))
	{
	}

	void FWaveTableSampler::Interpolate(const FWaveTableData& InTableData, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode)
	{
		switch (InTableData.GetBitDepth())
		{
			case EWaveTableBitDepth::IEEE_Float:
			{
				TArrayView<const float> InTableView;
				InTableData.GetDataView(InTableView);
				Interpolate(InTableView, InOutIndexToSamplesView, InterpMode);
			}
			break;

			case EWaveTableBitDepth::PCM_16:
			{
				TArrayView<const int16> InTableView;
				InTableData.GetDataView(InTableView);
				Interpolate(InTableView, InOutIndexToSamplesView, InterpMode);
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
				checkNoEntry();
			}
		}
	}

	void FWaveTableSampler::Interpolate(TArrayView<const int16> InTableView, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode)
	{
		using namespace UtilitiesPrivate;

		constexpr float ConversionValue = 1.0f / static_cast<float>(TNumericLimits<int16>::Max());
		switch (InterpMode)
		{
			case EInterpolationMode::None:
			{
				StepIndexInterpolator(InTableView, InOutIndexToSamplesView, ConversionValue);
			}
			break;

			case EInterpolationMode::Linear:
			{
				LinearIndexInterpolator(InTableView, InOutIndexToSamplesView, ConversionValue);
			}
			break;

			case EInterpolationMode::Cubic:
			{
				CubicIndexInterpolator(InTableView, InOutIndexToSamplesView, ConversionValue);
			}
			break;

			case EInterpolationMode::MaxValue:
			{
				MaxValueIndexInterpolator(InTableView, InOutIndexToSamplesView, ConversionValue);
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EInterpolationMode::COUNT) == 4, "Possible missing switch coverage for EInterpolationMode");
				checkNoEntry();
			}
			break;
		}
	}

	void FWaveTableSampler::Interpolate(TArrayView<const float> InTableView, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode)
	{
		using namespace UtilitiesPrivate;

		switch (InterpMode)
		{
			case EInterpolationMode::None:
			{
				StepIndexInterpolator(InTableView, InOutIndexToSamplesView);
			}
			break;

			case EInterpolationMode::Linear:
			{
				LinearIndexInterpolator(InTableView, InOutIndexToSamplesView);
			}
			break;

			case EInterpolationMode::Cubic:
			{
				CubicIndexInterpolator(InTableView, InOutIndexToSamplesView);
			}
			break;

			case EInterpolationMode::MaxValue:
			{
				MaxValueIndexInterpolator(InTableView, InOutIndexToSamplesView);
			}
			break;

			default:
			{
				static_assert(static_cast<int32>(EInterpolationMode::COUNT) == 4, "Possible missing switch coverage for EInterpolationMode");
				checkNoEntry();
			}
			break;
		}
	}

	void FWaveTableSampler::Reset()
	{
		LastIndex = 0.0f;
	}

	const FWaveTableSampler::FSettings& FWaveTableSampler::GetSettings() const
	{
		return Settings;
	}

	void FWaveTableSampler::SetFreq(float InFreq)
	{
		Settings.Freq = InFreq;
	}

	void FWaveTableSampler::SetInterpolationMode(EInterpolationMode InMode)
	{
		Settings.InterpolationMode = InMode;
	}

	void FWaveTableSampler::SetOneShot(bool bInOneShot)
	{
		Settings.bOneShot = bInOneShot;
	}

	void FWaveTableSampler::SetPhase(float InPhase)
	{
		Settings.Phase = InPhase;
	}

	float FWaveTableSampler::Process(const FWaveTableView& InTableView, float& OutSample, ESingleSampleMode InMode)
	{
		if(InTableView.IsEmpty())
		{
			return 0.f;
		}

		TArrayView<float> OutSamplesView { &OutSample, 1 };
		const float Index = Process(InTableView, OutSamplesView);
		return FinalizeSingleSample(Index, InTableView.Num(), OutSamplesView, InTableView.SampleView.Last(), InTableView.FinalValue, InMode);
	}

	float FWaveTableSampler::Process(const FWaveTableView& InTableView, TArrayView<float> OutSamplesView)
	{
		return Process(InTableView, { }, { }, { }, OutSamplesView);
	}

	float FWaveTableSampler::Process(const FWaveTableView& InTableView, TArrayView<const float> InFreqModulator, TArrayView<const float> InPhaseModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutSamplesView)
	{
		float CurViewLastIndex = 0.0f;

		if (!OutSamplesView.IsEmpty())
		{
			if (InTableView.IsEmpty())
			{
				Audio::ArraySetToConstantInplace(OutSamplesView, 0.0f);
			}
			else
			{
				// Compute index frequency & phase and stuff into samples view.
				// Interpolator will convert to associated values.
				ComputeIndexFrequency(InTableView.Num(), InFreqModulator, InSyncTriggers, OutSamplesView);
				ComputeIndexPhase(InTableView.Num(), InPhaseModulator, OutSamplesView);

				// Find stopping index if it is found and sampler is set to one shot,
				// and if set clear out section of samples view beyond end computed in prior steps.
				CurViewLastIndex = ComputeIndexFinished(InSyncTriggers, OutSamplesView);

				Interpolate(InTableView.SampleView, OutSamplesView, Settings.InterpolationMode);
			}

			if (!FMath::IsNearlyEqual(Settings.Amplitude, 1.0f))
			{
				Audio::ArrayMultiplyByConstantInPlace(OutSamplesView, Settings.Amplitude);
			}

			if (!FMath::IsNearlyZero(Settings.Offset))
			{
				Audio::ArrayAddConstantInplace(OutSamplesView, Settings.Offset);
			}
		}

		return CurViewLastIndex;
	}

	float FWaveTableSampler::Process(const FWaveTableData& InTableData, float& OutSample, ESingleSampleMode InMode)
	{
		if (InTableData.IsEmpty())
		{
			return 0.f;
		}

		TArrayView<float> OutSamplesView { &OutSample, 1 };
		const float Index = Process(InTableData, OutSamplesView);
		return FinalizeSingleSample(Index, InTableData.GetNumSamples(), OutSamplesView, InTableData.GetLastValue(), InTableData.GetFinalValue(), InMode);

	}

	float FWaveTableSampler::Process(const FWaveTableData& InTableData, TArrayView<float> OutSamplesView)
	{
		return Process(InTableData, { }, { }, { }, OutSamplesView);
	}

	float FWaveTableSampler::Process(const FWaveTableData& InTableData, TArrayView<const float> InFreqModulator, TArrayView<const float> InPhaseModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutSamplesView)
	{
		float CurViewLastIndex = 0.0f;

		if (!OutSamplesView.IsEmpty())
		{
			if (InTableData.IsEmpty())
			{
				Audio::ArraySetToConstantInplace(OutSamplesView, 0.0f);
			}
			else
			{
				// Compute index frequency & phase and stuff into samples view.
				// Interpolator will convert to associated values.
				ComputeIndexFrequency(InTableData.GetNumSamples(), InFreqModulator, InSyncTriggers, OutSamplesView);
				ComputeIndexPhase(InTableData.GetNumSamples(), InPhaseModulator, OutSamplesView);

				// Find stopping index if it is found and sampler is set to one shot,
				// and if set clear out section of samples view beyond end computed in prior steps.
				CurViewLastIndex = ComputeIndexFinished(InSyncTriggers, OutSamplesView);

				switch (InTableData.GetBitDepth())
				{
					case EWaveTableBitDepth::IEEE_Float:
					{
						TArrayView<const float> DataView;
						InTableData.GetDataView(DataView);
						Interpolate(DataView, OutSamplesView, Settings.InterpolationMode);
					}
					break;

					case EWaveTableBitDepth::PCM_16:
					{
						TArrayView<const int16> DataView;
						InTableData.GetDataView(DataView);
						Interpolate(DataView, OutSamplesView, Settings.InterpolationMode);
					}
					break;

					default:
					{
						static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
						checkNoEntry();
					}
				}
			}

			if (!FMath::IsNearlyEqual(Settings.Amplitude, 1.0f))
			{
				Audio::ArrayMultiplyByConstantInPlace(OutSamplesView, Settings.Amplitude);
			}

			if (!FMath::IsNearlyZero(Settings.Offset))
			{
				Audio::ArrayAddConstantInplace(OutSamplesView, Settings.Offset);
			}
		}

		return CurViewLastIndex;
	}

	void FWaveTableSampler::ComputeIndexFrequency(int32 NumInputSamples, TArrayView<const float> InFreqModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutIndicesView)
	{
		Audio::ArraySetToConstantInplace(OutIndicesView, Settings.Freq * NumInputSamples / OutIndicesView.Num());

		if (InFreqModulator.Num() == OutIndicesView.Num())
		{
			Audio::ArrayMultiplyInPlace(InFreqModulator, OutIndicesView);
		}

		auto TransformLastIndex = [this, NumInputSamples, &InFreqModulator, &InSyncTriggers, &OutIndicesView](float InputIndex)
		{
			if (InFreqModulator.Num() == OutIndicesView.Num())
			{
				return (InFreqModulator.Last() * Settings.Freq * NumInputSamples) + InputIndex;
			}
			else
			{
				checkf(InFreqModulator.IsEmpty(), TEXT("FreqModulator view should be the same size as the sample view or not supplied (size of 0)."));
				return (Settings.Freq * NumInputSamples) + InputIndex;
			}
		};

		float* OutSamples = OutIndicesView.GetData();
		if (InSyncTriggers.Num() == OutIndicesView.Num())
		{
			float SyncIndex = 0.0f;
			const float* SyncTrigData = InSyncTriggers.GetData();
			for (int32 i = 0; i < OutIndicesView.Num(); ++i)
			{
				// Numerically negate trig value as high (1.0f) should
				// be zero to "reset" index state (avoids conditional
				// in loop at expense of memory).
				const float InvertedTrig = -1.0f * SyncTrigData[i] + 1.0f;
				SyncIndex *= InvertedTrig;
				LastIndex *= InvertedTrig;
				OutSamples[i] *= SyncIndex;
				OutSamples[i] += LastIndex;

				SyncIndex++;
			}

			LastIndex = TransformLastIndex(LastIndex) + TransformLastIndex(SyncIndex);
		}
		else
		{
			checkf(InSyncTriggers.IsEmpty(), TEXT("SyncTriggers view should be the same size as the sample view or not supplied (size of 0)."));
			for (int32 i = 0; i < OutIndicesView.Num(); ++i)
			{
				OutSamples[i] *= i;
				OutSamples[i] += LastIndex;
			}

			LastIndex = TransformLastIndex(LastIndex);
		}

		LastIndex = FMath::Frac(LastIndex / NumInputSamples) * NumInputSamples;
	}

	void FWaveTableSampler::ComputeIndexPhase(int32 NumInputSamples, TArrayView<const float> InPhaseModulator, TArrayView<float> OutIndicesView)
	{
		check(!OutIndicesView.IsEmpty());

		if (InPhaseModulator.Num() == OutIndicesView.Num())
		{
			PhaseModScratch.SetNum(OutIndicesView.Num());
			Audio::ArraySetToConstantInplace(PhaseModScratch, Settings.Phase);
			Audio::ArrayAddInPlace(InPhaseModulator, PhaseModScratch);

			const float* PhaseModData = PhaseModScratch.GetData();
			float* OutSamples = OutIndicesView.GetData();
			for (int32 i = 0; i < OutIndicesView.Num(); ++i)
			{
				const float PhaseIndexOffset = FMath::Frac(PhaseModData[i]) * NumInputSamples;
				OutSamples[i] += PhaseIndexOffset;
			}
		}
		else
		{
			checkf(InPhaseModulator.IsEmpty(), TEXT("PhaseModulator view should be the same size as the sample view or not supplied (size of 0)."));

			const float PhaseIndexOffset = FMath::Frac(Settings.Phase) * NumInputSamples;
			Audio::ArrayAddConstantInplace(OutIndicesView, PhaseIndexOffset);
		}
	}

	float FWaveTableSampler::ComputeIndexFinished(TArrayView<const float> InSyncTriggers, TArrayView<float> OutIndicesView)
	{
		check(!OutIndicesView.IsEmpty());

		if (Settings.bOneShot)
		{
			bool Stopped = false;
			OneShotData.IndexFinished = 0;
			if (InSyncTriggers.IsEmpty())
			{
				for (int32 ArrayIndex = 0; ArrayIndex < OutIndicesView.Num(); ++ArrayIndex)
				{
					const float& ViewIndex = OutIndicesView[ArrayIndex];
					if (OneShotData.LastOutputIndex >= ViewIndex)
					{
						OneShotData.IndexFinished = ArrayIndex;
						return OneShotData.IndexFinished;
					}

					OneShotData.LastOutputIndex = ViewIndex;
				}
			}
			else
			{
				checkf(InSyncTriggers.Num() == OutIndicesView.Num(), TEXT("If SyncTriggers view is supplied, must match length of given indices view"))
				for (int32 ArrayIndex = 0; ArrayIndex < OutIndicesView.Num(); ++ArrayIndex)
				{
					const float& ViewIndex = OutIndicesView[ArrayIndex];
					const bool NotSyncing = InSyncTriggers[ArrayIndex] == 0.0f;
					if (NotSyncing && OneShotData.LastOutputIndex >= ViewIndex)
					{
						OneShotData.IndexFinished = ArrayIndex;
						return OneShotData.IndexFinished;
					}

					OneShotData.LastOutputIndex = ViewIndex;
				}
			}

			if (OutIndicesView.IsEmpty() || Stopped != 0)
			{
				OneShotData.LastOutputIndex = -1.0f;
				return OneShotData.IndexFinished;
			}
		}

		OneShotData.IndexFinished = INDEX_NONE;
		return OutIndicesView.Last();
	}

	float FWaveTableSampler::FinalizeSingleSample(float Index, int32 NumSamples, TArrayView<float> OutSampleView, float LastTableValue, float FinalValue, ESingleSampleMode InMode)
	{
		// Sampler functions by default cyclically and interpolates against initial value so
		// early out as following index check and potential interp re-evaluation isn't necessary.
		if (InMode == FWaveTableSampler::ESingleSampleMode::Loop)
		{
			return Index;
		}

		// If interpolating between last value and next, check
		// to see if interpolation needs to be re-evaluated.
		// This check is expensive in the context of buffer
		// eval and thus only supported in "single-sample" mode.
		const bool bIsFinalInterp = Index > NumSamples - 1;
		const bool bHasCompleted = Index == 0 && (Settings.Phase > 0.0f || LastIndex > 0);
		if (bIsFinalInterp || bHasCompleted)
		{
			float ModeValue = 0.f;

			switch (InMode)
			{
				case ESingleSampleMode::Zero:
				case ESingleSampleMode::Unit:
				{
					ModeValue = static_cast<float>(InMode);
				}
				break;

				case ESingleSampleMode::Hold:
				{
					ModeValue = FinalValue;
				}
				break;

				case ESingleSampleMode::Loop:
				{
					checkNoEntry(); // Should be handled above
				}
				break;

				default:
					break;
			}

			if (bIsFinalInterp)
			{
				TArray<float> FinalInterp = { LastTableValue, ModeValue };
				OutSampleView.Last() = FMath::Frac(Index);
				Interpolate(FinalInterp, OutSampleView, Settings.InterpolationMode);
			}
			else
			{
				OutSampleView.Last() = ModeValue;
			}

			if (!FMath::IsNearlyEqual(Settings.Amplitude, 1.0f))
			{
				OutSampleView.Last() *= Settings.Amplitude;
			}

			if (!FMath::IsNearlyZero(Settings.Offset))
			{
				OutSampleView.Last() += Settings.Offset;
			}
		}

		return Index;
	}

	int32 FWaveTableSampler::GetIndexFinished() const
	{
		return OneShotData.IndexFinished;
	}
} // namespace WaveTable
