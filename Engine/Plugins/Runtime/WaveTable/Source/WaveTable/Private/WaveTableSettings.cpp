// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTableSettings)

void FWaveTableSettings::GetEditSourceBounds(int32& OutSourceTopOffset, int32& OutSourceNumSamples) const
{
	OutSourceTopOffset = 0;
	OutSourceNumSamples = 0;

	const int32 TotalNumSamples = SourceData.GetNumSamples();
	if (TotalNumSamples > 0)
	{
		OutSourceNumSamples = TotalNumSamples;
		if (Top > 0.0f)
		{
			OutSourceTopOffset = WaveTable::RatioToIndex(Top, TotalNumSamples);
			OutSourceTopOffset = FMath::Min(OutSourceTopOffset, TotalNumSamples);
			OutSourceNumSamples -= OutSourceTopOffset;
		}

		if (Tail > 0.0f)
		{
			int32 SourceTailOffset = WaveTable::RatioToIndex(Tail, TotalNumSamples);
			SourceTailOffset = FMath::Min(SourceTailOffset, TotalNumSamples);
			OutSourceNumSamples -= SourceTailOffset;
		}

		OutSourceNumSamples = FMath::Max(OutSourceNumSamples, 0);
	}
}

TArrayView<const float> FWaveTableSettings::GetEditSourceView() const
{
#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	int32 SourceNumSamples = SourcePCMData.Num();
	if (SourceNumSamples == 0)
	{
		return { };
	}

	int32 SourceTopOffset = 0;
	if (Top > 0.0f)
	{
		SourceTopOffset = WaveTable::RatioToIndex(Top, SourceNumSamples);
		SourceNumSamples -= SourceTopOffset;
	}

	if (Tail > 0.0f)
	{
		const int32 SourceTailOffset = WaveTable::RatioToIndex(Tail, SourceNumSamples);
		SourceNumSamples -= SourceTailOffset;
	}

	if (SourceTopOffset < SourcePCMData.Num())
	{
		return { SourcePCMData.GetData() + SourceTopOffset, SourceNumSamples };
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITOR
	return { };
}

#if WITH_EDITOR
void FWaveTableSettings::VersionPCMData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!SourcePCMData.IsEmpty())
	{
		SourceData.SetRawData(EWaveTableBitDepth::IEEE_Float, TArrayView<uint8>((uint8*)(SourcePCMData.GetData()), SourcePCMData.Num() * sizeof(float)));
		SourcePCMData.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

namespace WaveTable
{
	int32 GetWaveTableSize(EWaveTableResolution InWaveTableResolution, EWaveTableCurve InCurve, int32 InMaxPCMSize)
	{
		checkNoEntry();
		return 0;
	}
} // namespace WaveTable
