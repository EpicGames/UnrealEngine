// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableTransform.h"

#include "Curves/CurveFloat.h"
#include "DSP/FloatArrayMath.h"
#include "WaveTableImporter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTableTransform)


void FWaveTableTransform::Apply(float& InOutValue, bool bInBipolar) const
{
	FWaveTableData TempTable(TArrayView<float>{ &InOutValue, 1 }, 0.0f);
	TArrayView<float> TempData;
	TempTable.GetDataView(TempData);

	// If no table data is cached, attempt to sample from curve
	if (TableData.IsEmpty())
	{
		bInBipolar ? SampleCurveBipolar(TempTable) : SampleCurveUnipolar(TempTable);
	}
	else
	{
		InOutValue *= TableData.GetNumSamples(); // Convert to float index
		TArrayView<const float> TableView;
		TableData.GetDataView(TableView);
		WaveTable::FWaveTableSampler::Interpolate(TableView, TempData);
	}

	InOutValue = TempData.Last();
}

void FWaveTableTransform::SampleCurveBipolar(FWaveTableData& InOutTableData) const
{
	checkf(InOutTableData.GetBitDepth() == EWaveTableBitDepth::IEEE_Float, TEXT("Curve sampling only supports IEEE 32-bit float bit depth"));

	TArrayView<float> TableView;
	InOutTableData.GetDataView(TableView);

	// Clamp the input
	Audio::ArrayClampInPlace(TableView, 0.0f, 1.0f);

	float* ValueData = TableView.GetData();
	switch (Curve)
	{
		case EWaveTableCurve::Custom:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = CurveCustom.Eval(ValueData[i]);
			}
				
			InOutTableData.SetFinalValue(CurveCustom.Eval(1.0f));
			break;
		}

		case EWaveTableCurve::Shared:
		{
			if (CurveShared)
			{
				for (int32 i = 0; i < TableView.Num(); ++i)
				{
					ValueData[i] = CurveShared->FloatCurve.Eval(ValueData[i]);
				}
				
				InOutTableData.SetFinalValue(CurveShared->FloatCurve.Eval(1.0f));
			}
			break;
		}

		case EWaveTableCurve::Linear_Inv:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = FMath::Lerp(1.0f, -1.0f, ValueData[i]);
			}
				
			InOutTableData.SetFinalValue(-1.0f);
		}
		break;

		case EWaveTableCurve::Linear:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = FMath::Lerp(-1.0f, 1.0f, ValueData[i]);
			}
			
			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Exp:
		{
			// Alpha is limited to between 0.0f and 1.0f and ExponentialScalar
			// between 0 and 10 to keep ValueData "sane" and avoid float boundary.
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] *= (FMath::Pow(10.0f, Scalar * (ValueData[i] - 1.0f)));
				ValueData[i] -= 0.5f;
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Exp_Inverse:
		{
			// Alpha is limited to between 0.0f and 1.0f and ExponentialScalar
			// between 0 and 10 to keep ValueData "sane" and avoid float boundary.
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = ((ValueData[i] - 1.0f) * FMath::Pow(10.0f, -1.0f * Scalar * ValueData[i])) + 1.0f;
				ValueData[i] -= 0.5f;
			}

			InOutTableData.SetFinalValue(-1.0f);
		}
		break;

		case EWaveTableCurve::Log:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = (Scalar * FMath::LogX(10.0f, ValueData[i])) + 0.5f;
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Sin:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = FMath::Sin(HALF_PI * ValueData[i]) - 0.5f;
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::SCurve:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = 0.5f * FMath::Sin(((PI * ValueData[i]) - HALF_PI));
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Sin_Full:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = FMath::Sin(2.0f * PI * ValueData[i]);
			}

			InOutTableData.SetFinalValue(0.0f);
		}
		break;

		case EWaveTableCurve::File:
		{
			SampleWaveTableData(InOutTableData);
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EWaveTableCurve::Count) == 11, "Possible missing case coverage for output curve.");
		}
		break;
	}

	Audio::ArrayClampInPlace(TableView, -1.0f, 1.0f);
}

void FWaveTableTransform::SampleCurveUnipolar(FWaveTableData& InOutTableData) const
{
	checkf(InOutTableData.GetBitDepth() == EWaveTableBitDepth::IEEE_Float, TEXT("Curve sampling only supports IEEE 32-bit float bit depth"));

	TArrayView<float> TableView;
	InOutTableData.GetDataView(TableView);
	
	if (ShouldClampToNormalizedRange())
	{
		Audio::ArrayClampInPlace(TableView, 0.0f, 1.0f);
	}

	float* ValueData = TableView.GetData();
	switch (Curve)
	{
		case EWaveTableCurve::Custom:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = CurveCustom.Eval(ValueData[i]);
			}

			InOutTableData.SetFinalValue(CurveCustom.Eval(1.0f));
			break;
		}

		case EWaveTableCurve::Shared:
		{
			if (CurveShared)
			{
				for (int32 i = 0; i < TableView.Num(); ++i)
				{
					ValueData[i] = CurveShared->FloatCurve.Eval(ValueData[i]);
				}
				InOutTableData.SetFinalValue(CurveShared->FloatCurve.Eval(1.0f));
			}
			break;
		}

		case EWaveTableCurve::Linear_Inv:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = 1.0f - TableView[i];
			}
				
			InOutTableData.SetFinalValue(0.0f);
		}
		break;

		case EWaveTableCurve::Linear:
		{
			// Do nothing, as output values == clamped input values
			InOutTableData.SetFinalValue(1.0f);
		}
		break;
	
		case EWaveTableCurve::Exp:
		{
			// Alpha is limited to between 0.0f and 1.0f and ExponentialScalar
			// between 0 and 10 to keep ValueData[i]s "sane" and avoid float boundary.
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] *= (FMath::Pow(10.0f, Scalar * (ValueData[i] - 1.0f)));
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Exp_Inverse:
		{
			// Alpha is limited to between 0.0f and 1.0f and ExponentialScalar
			// between 0 and 10 to keep ValueData[i]s "sane" and avoid float boundary.
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = ((ValueData[i] - 1.0f) * FMath::Pow(10.0f, -1.0f * Scalar * ValueData[i])) + 1.0f;
			}

			InOutTableData.SetFinalValue(0.0f);
		}
		break;

		case EWaveTableCurve::Log:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = (Scalar * FMath::LogX(10.0f, ValueData[i])) + 1.0f;
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Sin:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = FMath::Sin(HALF_PI * ValueData[i]);
			}
				
			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::SCurve:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = 0.5f * FMath::Sin(((PI * ValueData[i]) - HALF_PI)) + 0.5f;
			}

			InOutTableData.SetFinalValue(1.0f);
		}
		break;

		case EWaveTableCurve::Sin_Full:
		{
			for (int32 i = 0; i < TableView.Num(); ++i)
			{
				ValueData[i] = 0.5f * FMath::Sin(2.0f * PI * ValueData[i]) + 0.5f;
			}

			InOutTableData.SetFinalValue(0.0f);
		}
		break;

		case EWaveTableCurve::File:
		{
			SampleWaveTableData(InOutTableData);
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EWaveTableCurve::Count) == 11, "Possible missing case coverage for output curve.");
		}
		break;
	}

	if (ShouldClampToNormalizedRange())
	{
		Audio::ArrayClampInPlace(TableView, 0.0f, 1.0f);
	}
}

void FWaveTableTransform::SampleWaveTableData(FWaveTableData& InOutTableData) const
{
	checkf(InOutTableData.GetBitDepth() == EWaveTableBitDepth::IEEE_Float, TEXT("WaveTable resampling only supports IEEE 32-bit float bit depth"));

	TArrayView<float> TableView;
	InOutTableData.GetDataView(TableView);

	if (TableData.IsEmpty())
	{
		Audio::ArraySetToConstantInplace(TableView, 0.0f);
		InOutTableData.SetFinalValue(0.0f);
	}
	else
	{
		constexpr auto InterpMode = WaveTable::FWaveTableSampler::EInterpolationMode::MaxValue;
		WaveTable::FWaveTableSampler::Interpolate(TableData, TableView, InterpMode);
		InOutTableData.SetFinalValue(TableData.GetFinalValue());
	}
}

void FWaveTableTransform::CacheCurve()
{
	if (Curve == EWaveTableCurve::Shared && CurveShared)
	{
		Curve = EWaveTableCurve::Custom;
		CurveCustom = CurveShared->FloatCurve;
	}

	// Always clear shared curve regardless to avoid
	// holding a reference to the UObject when never
	// expected to be used.
	CurveShared = nullptr;
}

float FWaveTableTransform::GetFinalValue() const
{
	return TableData.GetFinalValue();
}

void FWaveTableTransform::SetFinalValue(float InValue)
{
}

const FWaveTableData& FWaveTableTransform::GetTableData() const
{
	return TableData;
}

#if WITH_EDITOR
void FWaveTableTransform::CopyToWaveTable(TArrayView<float> InOutTable, float& OutFinalValue, bool bInBipolar) const
{
	constexpr bool bIsLoop = false;

	FWaveTableData Data;
	Data.SetData(InOutTable, bIsLoop);
	Data.SetFinalValue(OutFinalValue);

	CopyToWaveTable(Data, bInBipolar);

	ensureAlwaysMsgf(Data.GetDataView(InOutTable), TEXT("BitDepth format mismatch"));
	OutFinalValue = Data.GetFinalValue();
}

void FWaveTableTransform::CopyToWaveTable(FWaveTableData& InOutTableData, bool bInBipolar) const
{
	FWaveTableData SampleData(EWaveTableBitDepth::IEEE_Float);
	SampleData.Zero(InOutTableData.GetNumSamples());

	TArrayView<float> SampleDataTableView;
	SampleData.GetDataView(SampleDataTableView);
	const float Delta = 1.0f / InOutTableData.GetNumSamples();
	for (int32 Index = 0; Index < SampleDataTableView.Num(); ++Index)
	{
		SampleDataTableView[Index] = Index * Delta;
	}

	bInBipolar ? SampleCurveBipolar(SampleData) : SampleCurveUnipolar(SampleData);

	switch (InOutTableData.GetBitDepth())
	{
		case EWaveTableBitDepth::IEEE_Float:
		{
			InOutTableData = MoveTemp(SampleData);
		}
		break;

		case EWaveTableBitDepth::PCM_16:
		{
			TArrayView<int16> TableView;
			InOutTableData.GetDataView(TableView);
			Audio::ArrayFloatToPcm16(SampleDataTableView, TableView);
		}
		break;

		default:
		{
			static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
			checkNoEntry();
		}
	}
}

void FWaveTableTransform::CreateWaveTable(FWaveTableData& InOutTableData, bool bInBipolar) const
{
	switch (Curve)
	{
		case EWaveTableCurve::File:
		{
			WaveTable::FImporter Importer(WaveTableSettings, bInBipolar);
			Importer.Process(InOutTableData);
		}
		break;

		case EWaveTableCurve::Custom:
		case EWaveTableCurve::Shared:
		case EWaveTableCurve::Exp:
		case EWaveTableCurve::Exp_Inverse:
		case EWaveTableCurve::Linear:
		case EWaveTableCurve::Linear_Inv:
		case EWaveTableCurve::Log:
		case EWaveTableCurve::SCurve:
		case EWaveTableCurve::Sin:
		case EWaveTableCurve::Sin_Full:
		default:
		{
			CopyToWaveTable(InOutTableData, bInBipolar);
			InOutTableData.SetFinalValue(TableData.GetFinalValue());
		}
		break;
	}
}

void FWaveTableTransform::CreateWaveTable(TArray<float>& InOutTable, float& OutFinalValue, bool bInBipolar) const
{
	constexpr bool bIsLoop = false;

	FWaveTableData TempTableData;
	TempTableData.SetData(InOutTable, bIsLoop);
	TempTableData.SetFinalValue(OutFinalValue);

	CreateWaveTable(TempTableData, bInBipolar);

	TArrayView<float> OutTableView;
	TempTableData.GetDataView(OutTableView);
	OutFinalValue = TableData.GetFinalValue();
	InOutTable = MoveTemp(OutTableView);
}
#endif // WITH_EDITOR
