// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"
#include "WaveTableSampler.h"

#include "WaveTable.generated.h"

#define UE_API WAVETABLE_API


namespace WaveTable
{
	// An invalid WaveTable value, used to detect uninitialized final values.
	constexpr float InvalidWaveTableValue = TNumericLimits<float>::Min();

	// Read-only view to a wave table struct
	struct FWaveTableView
	{
		TArrayView<const float> SampleView;
		const float FinalValue = 0.0f;

		UE_API explicit FWaveTableView(const TArray<float>& InSamples, const float InFinalValue);
		UE_API explicit FWaveTableView(const TArrayView<const float>& InSamples, const float InFinalValue);

		// returns the length of the SampleView array, not including the additional FinalValue
		UE_API int32 Num() const;
		UE_API bool IsEmpty() const;
	};

	// WaveTable data used at runtime by oscillators, envelopes,
	//  etc.  Always as 32-bit float bit depth for processing optimization,
	// at the expense of memory.
	struct FWaveTable
	{
	private:
		TArray<float> Samples;
		float FinalValue = 0.0f;

	public:
		FWaveTable() = default;
		UE_API FWaveTable(const TArray<float>& InSamples, const float InFinalValue = 0.0f);
		UE_API FWaveTable(TArray<float>&& InSamples, const float InFinalValue);
		UE_API FWaveTable(const ::FWaveTableData& InData);

		UE_API float* GetData();
		UE_API const float* GetData() const;
		UE_API FWaveTableView GetView() const;
		UE_API TArrayView<float> GetSamples();
		UE_API int32 Num() const;
		UE_API void Set(TArray<float>&& InSamples);
		UE_API void Set(TArrayView<const float> InSamples);
		UE_API void SetFinalValue(const float InValue);
		UE_API void SetNum(int32 InSize);
		UE_API void Zero();

		inline static void WrapIndexSmooth(int32 InMax, float& InOutIndex)
		{
			InOutIndex = FMath::Abs(InOutIndex); // Avoids fractional offset flip at 0 crossing
			const int32 WrapIndex = FMath::TruncToInt32(InOutIndex) % InMax;
			const float Fractional = FMath::Frac(InOutIndex);
			InOutIndex = WrapIndex + Fractional;
		}
	};
} // namespace WaveTable


UENUM(BlueprintType)
enum class EWaveTableBitDepth : uint8
{
	// Lower resolution and marginal performance cost with
	// conversion overhead (engine operates on 32-bit)
	// with the advantage of half the size in memory.
	PCM_16,

	// Higher precision and faster operative performance
	// (engine operates at 32-bit) at the cost of twice the
	// memory of 16-bit.
	IEEE_Float,

	COUNT UMETA(Hidden)
};


// Serialized WaveTable data, that supports multiple bit depth formats.
USTRUCT(BlueprintType)
struct FWaveTableData
{
	GENERATED_BODY();

	FWaveTableData() = default;
	UE_API FWaveTableData(EWaveTableBitDepth InBitDepth);
	UE_API FWaveTableData(TArrayView<const float> InSamples, float InFinalValue);
	UE_API FWaveTableData(TArrayView<const int16> InSamples, float InFinalValue);

#if WITH_EDITOR
	static UE_API FName GetBitDepthPropertyName();
#endif // WITH_EDITOR

private:
	UPROPERTY(EditAnywhere, Category = Options)
	EWaveTableBitDepth BitDepth = EWaveTableBitDepth::PCM_16;

	UPROPERTY()
	TArray<uint8> Data;

	UPROPERTY()
	float FinalValue = ::WaveTable::InvalidWaveTableValue;

	// Returns the size of the underlying data's sample in number of bytes.
	inline int32 GetSampleSize() const;

public:
	// Adds this WaveTableData's internal contents to the given float buffer view,
	// initially performing bit depth conversion to values to IEEE_FLOAT if required.
	// Provided buffer's size must match this WaveTableData's samples count.
	UE_API void ArrayMixIn(TArrayView<float> OutputWaveSamples, float Gain = 1.0f) const;

	// Empties the underlying data container, deallocating memory and invalidating the FinalValue.
	UE_API void Empty();

	UE_API EWaveTableBitDepth GetBitDepth() const;

	// Returns read-only view of byte array that stores sample data.
	UE_API const TArray<uint8>& GetRawData() const;

	// Returns true if ArrayView of underlying Data in the proper sample format,
	// setting OutData to a view of the Table's data. Returns false if the bit
	// depth of the provided ArrayView does not match the Table's BitDepth property.
	UE_API bool GetDataView(TArrayView<const float>& OutData) const;
	UE_API bool GetDataView(TArrayView<float>& OutData);
	UE_API bool GetDataView(TArrayView<const int16>& OutData) const;
	UE_API bool GetDataView(TArrayView<int16>& OutData);

	// Returns number of samples within the underlying DataView, not including a FinalValue if set.
	// (Should not to be confused with the number of bytes used to represent the given DataView in
	// the internal Data container).
	UE_API int32 GetNumSamples() const;

	// Returns the last data value value as a float.  If DataView is empty, returns 0.
	UE_API float GetLastValue() const;

	// Returns the value to interpolate to beyond the last value of the DataView.
	// If FinalValue property is set to invalid (default), returns the last value in the 
	// underlying DataView (see GetLastValue).
	UE_API float GetFinalValue() const;

	// Whether or not the underlying data is empty.
	UE_API bool IsEmpty() const;

	// Resamples the underlying data to a provided new sample rate.
	UE_API bool Resample(int32 InCurrentSampleRate, int32 InNewSampleRate, ::WaveTable::FWaveTableSampler::EInterpolationMode InInterpMode = ::WaveTable::FWaveTableSampler::EInterpolationMode::Cubic);

	// Resets the underlying data to the given number of samples. Does not change
	// memory allocation of data unless NumSamples is larger than the current size.
	UE_API void Reset(int32 NumSamples);

	// Converts bit depth and sets internal data to the requested bit depth.
	// Returns true if conversion took place, false if not (i.e. data was 
	// already sampled at the given bit depth).
	UE_API bool SetBitDepth(EWaveTableBitDepth InBitDepth);
	UE_API void SetData(TArrayView<const int16> InData, bool bIsLoop);
	UE_API void SetData(TArrayView<const float> InData, bool bIsLoop);
	UE_API void SetRawData(EWaveTableBitDepth InBitDepth, TArrayView<const uint8> InData);
	UE_API void SetRawData(EWaveTableBitDepth InBitDepth, TArray<uint8>&& InData);
	UE_API void SetFinalValue(float InFinalValue);

	// Zeros the underlying data. If InNumSamples is set (optional),
	// allocates space in underlying data array for the given number of samples.
	UE_API void Zero(int32 InNumSamples = INDEX_NONE);
};

#undef UE_API
