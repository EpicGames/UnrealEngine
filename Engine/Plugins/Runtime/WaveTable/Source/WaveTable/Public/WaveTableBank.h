// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioProxyInitializer.h"
#include "WaveTable.h"
#include "WaveTableTransform.h"

#include "WaveTableBank.generated.h"

#define UE_API WAVETABLE_API


// Forward Declarations
struct FPropertyChangedEvent;
class FWaveTableBankAssetProxy;

using FWaveTableBankAssetProxyPtr = TSharedPtr<FWaveTableBankAssetProxy, ESPMode::ThreadSafe>;


USTRUCT()
struct FWaveTableBankEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Table)
	FWaveTableTransform Transform;
};

UCLASS(MinimalAPI, config = Engine, editinlinenew, BlueprintType)
class UWaveTableBank : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:
	// Sampling mode used for the bank.
	UPROPERTY(EditAnywhere, Category = Options)
	EWaveTableSamplingMode SampleMode = EWaveTableSamplingMode::FixedResolution;

	// Number of samples cached for each entry in the given bank.
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SampleMode == EWaveTableSamplingMode::FixedResolution", EditConditionHides))
	EWaveTableResolution Resolution = EWaveTableResolution::Res_256;

	// Number of samples cached for each entry in the given bank.
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SampleMode == EWaveTableSamplingMode::FixedSampleRate", ClampMin = "1", ClampMax = "48000", EditConditionHides))
	int32 SampleRate = 48000;

	// Determines if output from curve/wavetable are to be clamped between 
	// [-1.0f, 1.0f] (i.e. for waveform generation, oscillation, etc.)
	// or [0.0f, 1.0f] (i.e. for enveloping) when sampling curve/wavetable
	UPROPERTY(EditAnywhere, Category = Options)
	bool bBipolar = true;

#if WITH_EDITORONLY_DATA
	// Sum total size of all WaveTable data within the given bank
	UPROPERTY(VisibleAnywhere, Category = Options, meta = (DisplayName = "WaveTable Size (MB)"))
	float WaveTableSizeMB = 0.0f;

	// Length of all WaveTable samples in bank in seconds (at 48kHz)
	UE_DEPRECATED(5.3, "Samples now each have own length, as they no longer are required being the same length if using shared 'SampleRate' mode")
	float WaveTableLengthSec = 0.0f;
#endif // WITH_EDITORONLY_DATA


	/** Tables within the given bank */
	UE_DEPRECATED(5.3, "Direct access of 'Entries' will become protected member in future release and not externally modifiable in runtime builds. If in editor, use GetEntries(). "
		"To reduce memory consumption, entries are now readonly and cleared in runtime builds when proxy is generated.")
	UPROPERTY(EditAnywhere, Category = Options)
	TArray<FWaveTableBankEntry> Entries;

public:
	/* IAudioProxyDataFactory Implementation */
	UE_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API TArray<FWaveTableBankEntry>& GetEntries();

#if WITH_EDITOR
	UE_API void RefreshWaveTables();

	UE_API virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
	void CopyToProxyData();
	void MoveToProxyData();

	FWaveTableBankAssetProxyPtr ProxyData;
};

class FWaveTableBankAssetProxy : public Audio::TProxyData<FWaveTableBankAssetProxy>, public TSharedFromThis<FWaveTableBankAssetProxy, ESPMode::ThreadSafe>
{
public:
	IMPL_AUDIOPROXY_CLASS(FWaveTableBankAssetProxy);

	FWaveTableBankAssetProxy() = default;
	UE_API FWaveTableBankAssetProxy(uint32 InObjectId, EWaveTableSamplingMode InSamplingMode, int32 InSampleRate, const TArray<FWaveTableBankEntry>& InBankEntries, bool bInBipolar = false);
	UE_API FWaveTableBankAssetProxy(uint32 InObjectId, EWaveTableSamplingMode InSamplingMode, int32 InSampleRate, TArray<FWaveTableBankEntry>&& InBankEntries, bool bInBipolar = false);

	UE_DEPRECATED(5.3, "Proxy generation & respective data translation is now entirely handled by UWaveTableBank calling the constructor variants above.")
	UE_API FWaveTableBankAssetProxy(const UWaveTableBank& InWaveTableBank);

	virtual ~FWaveTableBankAssetProxy() = default;

	virtual EWaveTableSamplingMode GetSampleMode() const
	{
		return SampleMode;
	}

	virtual int32 GetSampleRate() const
	{
		return SampleRate;
	}

	virtual const TArray<FWaveTableData>& GetWaveTableData() const
	{
		return WaveTableData;
	}

	virtual bool IsBipolar() const
	{
		return bBipolar;
	}

	uint32 GetObjectId() const
	{
		return ObjectId;
	}

	friend inline uint32 GetTypeHash(const FWaveTableBankAssetProxy& InProxy)
	{
		return InProxy.GetObjectId();
	}

protected:
	bool bBipolar = false;
	uint32 ObjectId = INDEX_NONE;
	int32 SampleRate = 48000;
	EWaveTableSamplingMode SampleMode = EWaveTableSamplingMode::FixedResolution;
	TArray<FWaveTableData> WaveTableData;
};

#undef UE_API
