// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WaveTable.h"
#include "WaveTableSampler.h"
#include "WaveTableSettings.h"

#define UE_API WAVETABLE_API

// Forward Declarations
struct FWaveTableData;


namespace WaveTable
{
	class FImporter
	{
	public:
		UE_API FImporter(const FWaveTableSettings& InSettings, bool bInBipolar);

		UE_DEPRECATED(5.3, "Importer now supports multiple bit depths and sample rate. Use version of function that takes in an FWaveTableData struct.")
		UE_API void Process(TArray<float>& OutWaveTable);

		UE_API void Process(FWaveTableData& OutData);

	private:
		void ProcessInternal(const TArrayView<const float> InEditSourceView, TArray<float>& OutWaveTable);

		const FWaveTableSettings& Settings;

		bool bBipolar = false;

		FWaveTableSampler Sampler;
	};
} // namespace WaveTable

#undef UE_API
