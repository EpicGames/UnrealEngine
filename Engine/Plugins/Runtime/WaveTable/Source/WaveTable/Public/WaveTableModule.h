// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#define UE_API WAVETABLE_API


namespace WaveTable
{
class FModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
};
} // namespace WaveTable

#undef UE_API
