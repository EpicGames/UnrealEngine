// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"

#define UE_API WAVETABLE_API


struct FWaveTableData;

namespace WaveTable
{
	struct FWaveTableView;

	class FWaveTableSampler
	{
	public:
		// Interpolation mode when sampling values between indices
		enum class EInterpolationMode
		{
			// No interpolation (Stepped)
			None,

			// Linear interpolation
			Linear,

			// Cubic interpolation
			Cubic,

			// Takes the maximum value between two points (EXPENSIVE, but good for caching/drawn curves).
			MaxValue,

			COUNT
		};

		// Mode of interpolation between last value in table and subsequent input index when sampling single values.
		enum class ESingleSampleMode
		{
			// Interpolates last value to zero (0.0f) in table if sampled index is beyond last position
			Zero = 0,

			// Interpolates last value to unit (1.0f) in table if sampled index is beyond last position
			Unit = 1,

			// Holds last value in table if index is beyond last position
			Hold,

			// Interpolates last value and first value in table if sampled index is beyond last position
			Loop,
		};

		struct FSettings
		{
			float Amplitude = 1.0f;
			float Offset = 0.0f;

			// How many times to read through the incoming
			// buffer as a ratio to a single output buffer
			float Freq = 1.0f;

			// Offset from beginning to end of array [0.0, 1.0]
			float Phase = 0.0f;

			EInterpolationMode InterpolationMode = EInterpolationMode::Linear;

			bool bOneShot = false;
		};

		UE_API FWaveTableSampler();
		UE_API FWaveTableSampler(FSettings&& InSettings);

		// Interpolates and converts values in the given table for each provided index in the index-to-samples TArrayView (an array of sub-sample, floating point indices)
		static UE_API void Interpolate(const FWaveTableData& InTableData, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode = EInterpolationMode::Linear);
		static UE_API void Interpolate(TArrayView<const int16> InTableView, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode = EInterpolationMode::Linear);
		static UE_API void Interpolate(TArrayView<const float> InTableView, TArrayView<float> InOutIndexToSamplesView, EInterpolationMode InterpMode = EInterpolationMode::Linear);

		// Retrieves the sample value at the currently set phase, returning the floating point index which the current phase corresponds to.
		UE_API float Process(const FWaveTableView& InTableView, float& OutSample, ESingleSampleMode InMode = ESingleSampleMode::Zero);
		UE_API float Process(const FWaveTableData& InTableData, float& OutSample, ESingleSampleMode InMode = ESingleSampleMode::Zero);

		// Resamples entire table length into given view
		UE_API float Process(const FWaveTableView& InTableView, TArrayView<float> OutSamplesView);
		UE_API float Process(const FWaveTableData& InTableData, TArrayView<float> OutSamplesView);
		UE_API float Process(const FWaveTableView& InTableView, TArrayView<const float> InFreqModulator, TArrayView<const float> InPhaseModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutSamplesView);
		UE_API float Process(const FWaveTableData& InTableData, TArrayView<const float> InFreqModulator, TArrayView<const float> InPhaseModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutSamplesView);

		// Resets the last index operated on, restarting the interpolation process for subsequent calls.
		UE_API void Reset();

		UE_API const FSettings& GetSettings() const;

		// If set to one-shot, returns the first index where a reverse or stall occurred in the provided
		// interpolated index, ignoring samples where a "sync" request took place. If set to looping,
		// always INDEX_NONE as looping does not support a "finished" state.
		UE_API int32 GetIndexFinished() const;

		UE_API void SetInterpolationMode(EInterpolationMode InMode);
		UE_API void SetFreq(float InFreq);
		UE_API void SetOneShot(bool bInLooping);
		UE_API void SetPhase(float InPhase);

	private:
		void ComputeIndexFrequency(int32 NumInputSamples, TArrayView<const float> InFreqModulator, TArrayView<const float> InSyncTriggers, TArrayView<float> OutIndicesView);
		void ComputeIndexPhase(int32 NumInputSamples, TArrayView<const float> InPhaseModulator, TArrayView<float> OutIndicesView);

		// If set to one-shot, sets IndexFinished to valid index and resets remaining indices provided past the initial stop to the first index.
		// Returns last positively increasing index if finished. If not finished, returns the last index value stored in the view.
		float ComputeIndexFinished(TArrayView<const float> InSyncTriggers, TArrayView<float> OutIndicesView);

		// Utility that takes in necessary data for finalizing a single sample index processing from either a TableData or a TableView.  Covers edge cases for final value.
		float FinalizeSingleSample(float Index, int32 NumSamples, TArrayView<float> OutSample, float LastTableValue, float FinalValue, FWaveTableSampler::ESingleSampleMode InMode);

		float LastIndex = 0.0f;

		struct FOneShotData
		{
			// Cached last floating index from prior process, used to compute stopping index if forward progress terminates on buffer boundary.
			float LastOutputIndex = -1.0f;

			// Contains index where sampler finished as a result of a sample ending or phase reversing within the last process call.
			// (If < 0, no finish/phase reverse was detected and sampler is considered to be continuing to make forward progress)
			int32 IndexFinished = INDEX_NONE;
		} OneShotData;

		TArray<float> PhaseModScratch;
		FSettings Settings;

	};
} // namespace WaveTable

#undef UE_API
