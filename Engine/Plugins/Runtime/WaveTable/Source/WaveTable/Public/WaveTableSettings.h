// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WaveTable.h"

#include "WaveTableSettings.generated.h"

#define UE_API WAVETABLE_API


// Forward Declarations
class UWaveTableBank;
struct FWaveTableBankEntry;


namespace WaveTable
{
	// Converts a ratio [0.0, 1.0] to an index within the given number of array elements.
	[[nodiscard]] static inline int32 RatioToIndex(float InRatio, int32 InArrayNum)
	{
		check(InArrayNum > 0);
		check(InRatio >= 0.0f);

		return FMath::Min(InArrayNum - 1, FMath::TruncToInt32(InRatio * InArrayNum));
	};
} // namespace WaveTable


UENUM(BlueprintType)
enum class EWaveTableResolution : uint8
{
	None		= 0 UMETA(Hidden),

	Res_8		= 3 UMETA(DisplayName = "8"),
	Res_16		= 4 UMETA(DisplayName = "16"),
	Res_32		= 5 UMETA(DisplayName = "32"),
	Res_64		= 6 UMETA(DisplayName = "64"),
	Res_128		= 7 UMETA(DisplayName = "128"),
	Res_256		= 8 UMETA(DisplayName = "256"),
	Res_512		= 9 UMETA(DisplayName = "512"),
	Res_1024	= 10 UMETA(DisplayName = "1024"),
	Res_2048	= 11 UMETA(DisplayName = "2048"),
	Res_4096	= 12 UMETA(DisplayName = "4096"),

	// Maximum "static" power-of-two WaveTable resolution
	Res_Max		= Res_4096 UMETA(Hidden),

	// Takes largest source file length (if WaveTable was produced
	// by a file, PCM stream, SoundWave, etc.) of superset of waves,
	// or Res_Max if all superset of waves were produced by curves. This
	// can be very memory intensive if source file is large!
	Maximum
};


// SamplingMode of a given bank or collection of WaveTables
UENUM()
enum class EWaveTableSamplingMode : uint8
{
	// Enforces fixed sample rate for all members in the bank/collection,
	// enabling them to be of unique duration/number of samples. Good for
	// use cases when entries are being treated as separate, discrete but
	// related audio to be played back at a shared speed (ex. traditional
	// "samplers" or granulation).
	FixedSampleRate,

	// Enforces resolution (i.e. size of all tables), uniformly resampling
	// all tables in the collection to be the same length/number of samples
	// (if not already).  Supports use cases where systems are mixing/
	// interpolating or spatializing entries in lockstep (ex. oscillating
	// or enveloping).
	FixedResolution,

	COUNT UMETA(Hidden)
};


USTRUCT()
struct FWaveTableSettings
{
	GENERATED_USTRUCT_BODY()

	// File to import
	UPROPERTY(EditAnywhere, Category = Options, meta = (FilePathFilter = "Audio Files (*.aif, *.aiff, *.flac, *.ogg, *.wav)|*.aif;*.aiff;*.flac;*.ogg;*.wav"))
	FFilePath FilePath;

	// Index of channel in file to build WaveTable from (wraps if channel is greater than number in file)
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0"))
	int32 ChannelIndex = 0;

	// Source data last imported from the source file
	UPROPERTY(EditAnywhere, Category = Options)
	FWaveTableData SourceData;

	// Source sample rate from last imported source file
	UPROPERTY(VisibleAnywhere, Category = Options)
	int32 SourceSampleRate = 48000;

	// Percent to phase shift of table
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Phase = 0.0f;

	// Percent to remove from beginning of sampled WaveTable.
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Top = 0.0f;

	// Percent to remove from end of sampled WaveTable.
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Tail = 0.0f;

	// Percent to fade in over.
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float FadeIn  = 0.0f;

	// Percent to fade out over.
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float FadeOut = 0.0f;

	// Whether or not to normalize the WaveTable.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayAfter = "SourceData"))
	bool bNormalize = true;

	// Whether or not to remove offset from original file
	// (analogous to "DC offset" in circuit theory).
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayAfter = "bNormalize"))
	bool bRemoveOffset = false;

	// Whether or not an edit is contained to omit or modify source file's PCM data
	bool ContainsEdit() const
	{
		return Phase > 0.0f ||
			Top > 0.0f || Tail > 0.0f ||
			FadeIn > 0.0f || FadeOut > 0.0f ||
			bNormalize || bRemoveOffset;
	}

#if WITH_EDITOR
	UE_API void VersionPCMData();
#endif // WITH_EDITOR

	// Returns the respective offset & number of samples that
	// characterize the source data to apply light edits to.
	UE_API void GetEditSourceBounds(int32& OutSourceTopOffset, int32& OutSourceNumSamples) const;

	UE_DEPRECATED(5.3, "Source now supports multiple bit depths. Use 'GetEditSourceBounds' & apply to SourceData WaveTableData TArrayView in proper format using 'GetDataView'.")
	UE_API TArrayView<const float> GetEditSourceView() const;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (Deprecated = "5.3", DeprecationMessage = "Migrated to FWaveTableData member 'SourceData' to support multiple bitdepths"))
	TArray<float> SourcePCMData;
#endif // WITH_EDITORONLY_DATA
};

UENUM(BlueprintType)
enum class EWaveTableCurve : uint8
{
	Linear		UMETA(DisplayName = "Linear (Ramp In)"),
	Linear_Inv	UMETA(DisplayName = "Linear (Ramp Out)"),
	Exp			UMETA(DisplayName = "Exponential"),
	Exp_Inverse UMETA(DisplayName = "Exponential (Inverse)"),
	Log			UMETA(DisplayName = "Log"),

	Sin			UMETA(DisplayName = "Sin (90 deg)"),
	Sin_Full	UMETA(DisplayName = "Sin (360 deg)"),
	SCurve		UMETA(DisplayName = "Sin (+/- 90 deg)"),

	// Reference a shared curve asset
	Shared		UMETA(DisplayName = "Curve Asset"),

	// Design a custom curve unique to the owning transform
	Custom		UMETA(DisplayName = "Custom"),

	// Generate WaveTable from audio file
	File		UMETA(DisplayName = "File"),

	Count		UMETA(Hidden),
};

namespace WaveTable
{
	UE_DEPRECATED(5.3, "Removed from public API as banks now support different bit depths and sampling mode (i.e. sample count can no longer be conflated with generic size metric)")
	WAVETABLE_API int32 GetWaveTableSize(EWaveTableResolution InWaveTableResolution, EWaveTableCurve InCurve, int32 InMaxPCMSize = INDEX_NONE);
} // namespace WaveTable

#undef UE_API
