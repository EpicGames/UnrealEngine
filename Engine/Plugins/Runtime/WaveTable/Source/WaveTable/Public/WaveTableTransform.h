// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"
#include "Curves/RichCurve.h"
#include "WaveTable.h"
#include "WaveTableSettings.h"

#include "WaveTableTransform.generated.h"

#define UE_API WAVETABLE_API

class UCurveFloat;


USTRUCT(BlueprintType)
struct FWaveTableTransform
{
	GENERATED_USTRUCT_BODY()

	virtual ~FWaveTableTransform() = default;
	
	/** The curve to apply when transforming the output. */
	UPROPERTY(EditAnywhere, Category = Input, BlueprintReadWrite, meta = (DisplayName = "Curve Type"))
	EWaveTableCurve Curve = EWaveTableCurve::Linear;

	/** When curve set to log, exponential or exponential inverse, value is factor 'b' in following equations with output 'y' and input 'x':
	 *  Exponential: y = x * 10^-b(1-x)
	 *  Exponential (Inverse): y = ((x - 1) * 10^(-bx)) + 1
	 *  Logarithmic: y = b * log(x) + 1
	 */
	UPROPERTY(EditAnywhere, Category = Input, BlueprintReadWrite, meta = (DisplayName = "Exponential Scalar", ClampMin = "0.1", ClampMax = "10.0", UIMin = "0.1", UIMax = "10.0"))
	float Scalar = 2.5f;

	/** Custom curve to apply if output curve type is set to 'Custom.' */
	UPROPERTY()
	FRichCurve CurveCustom;

	/** Asset curve reference to apply if output curve type is set to 'Shared.' */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Asset"), Category = Curve, BlueprintReadWrite)
	TObjectPtr<UCurveFloat> CurveShared = nullptr;

protected:
	UPROPERTY()
	FWaveTableData TableData;

	// Duration of curve or file.  Only valid if parent SampleRate is set and SamplingMode is set to 'FixedSampleRate'
	// (If set to 'FixedResolution', duration is variable depending on the resolution (number of samples in the table data)
	// and device's sample rate.
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Duration (Sec)", Category = Input, ClampMin = "0.001", UIMin = "0.001"))
	float Duration = 1.0f;
	
	virtual bool ShouldClampToNormalizedRange() const { return true; }

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = WaveTable)
	FWaveTableSettings WaveTableSettings;
#endif // WITH_EDITORONLY_DATA

	// Applies transform to provided index value, setting it to the corresponding value.
	UE_API void Apply(float& InOutValue, bool bInBipolar = false) const;

	float GetDuration() const { return Duration; }

#if WITH_EDITOR
	UE_DEPRECATED(5.3, "Use FWaveTableData version instead")
	UE_API void CopyToWaveTable(TArrayView<float> InOutTable, float& OutFinalValue, bool bInBipolar) const;

	UE_DEPRECATED(5.3, "Use FWaveTableData version instead")
	UE_API void CreateWaveTable(TArray<float>& InOutTable, float& OutFinalValue, bool bInBipolar) const;

	// Copies a single, period [0.0, 1.0] to provided WaveTableData using this transform's curve. The
	// size of the resulting WaveTable is the provided ArrayView's size. If curve is itself
	// set to 'File' and internally cached as WaveTable, will re-sample using max value
	// interpolation. Does *not* apply destructive edits from WaveTableSetting's source table data.
	UE_API void CopyToWaveTable(FWaveTableData& InOutTableData, bool bInBipolar) const;

	// Creates WaveTableData from transform respective Curve settings.
	// Applies destructive edits as provided by WaveTableSettings if curve set to 'File'.
	UE_API void CreateWaveTable(FWaveTableData& InOutTableData, bool bInBipolar) const;

	static FName GetDurationPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FWaveTableTransform, Duration);
	}

	// Versions deprecated "WaveTable" property data to PCMData, to support serialization of different bit depths
	void VersionTableData()
	{
		WaveTableSettings.VersionPCMData();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!WaveTable.IsEmpty())
		{
			constexpr bool bIsLoop = false;
			TableData.SetData(WaveTable, bIsLoop);
			TableData.SetFinalValue(FinalValue);

			FinalValue = WaveTable::InvalidWaveTableValue;
			WaveTable.Empty();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITOR

	/** Caches curve data.  Should a shared curve be selected, curve points are copied locally
	  * to CurveCustom and Curve type is set accordingly.  Can be used to both avoid keeping a
	  * uobject reference should the transform be used on a non-game thread and potentially to
	  * further customize a referenced curve locally.
	  */
	UE_API void CacheCurve();

	UE_API const FWaveTableData& GetTableData() const;

	UE_DEPRECATED(5.3, "FinalValue has moved to TableData. Use property's FinalValue get accessor directly.")
	UE_API float GetFinalValue() const;

	UE_DEPRECATED(5.3, "FinalValue has moved to TableData and is read-only.")
	UE_API void SetFinalValue(float InValue);

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (Deprecated = "5.3", DeprecationMessage = "WaveTable data now supports multiple bit depths and has moved to TableData"))
	TArray<float> WaveTable;

	UPROPERTY(meta = (Deprecated = "5.3", DeprecationMessage = "WaveTable data now supports multiple bit depths and FinalValue has moved to TableData"))
	float FinalValue = 0.0f;
#endif // WITH_EDITORONLY_DATA

private:
	/** Clamps & applies transform to provided values as bipolar signal*/
	UE_API void SampleCurveBipolar(FWaveTableData& InOutTableData) const;

	/** Clamps & applies transform to provided values as unipolar signal */
	UE_API void SampleCurveUnipolar(FWaveTableData& InOutTableData) const;

	/** Converts internal TableData to float bit rate if not and re-samples to fit into provided wavetable data. */
	UE_API void SampleWaveTableData(FWaveTableData& InOutTableData) const;

	friend class FWaveTableBankAssetProxy;
	friend class UWaveTableBank;
};

#undef UE_API
