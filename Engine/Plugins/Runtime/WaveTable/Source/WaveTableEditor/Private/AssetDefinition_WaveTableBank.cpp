// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_WaveTableBank.h"
#include "WaveTableBankEditor.h"

EAssetCommandResult UAssetDefinition_WaveTableBank::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace WaveTable::Editor;
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UWaveTableBank* Patch : OpenArgs.LoadObjects<UWaveTableBank>())
	{
		TSharedRef<FBankEditor> Editor = MakeShared<FBankEditor>();
		Editor->Init(Mode, OpenArgs.ToolkitHost, Patch);
	}

	return EAssetCommandResult::Handled;
}
