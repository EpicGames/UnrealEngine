// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "WaveTableBank.h"
#include "AssetDefinition_WaveTableBank.generated.h"

UCLASS()
class UAssetDefinition_WaveTableBank : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_WaveTableBank", "WaveTable Bank"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(232, 122, 0, 255); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UWaveTableBank::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_WaveTableBankSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_WaveTableBankSubMenuSection", "DSP Effects and Synthesis"), ECategoryMenuType::Section))
		};
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
