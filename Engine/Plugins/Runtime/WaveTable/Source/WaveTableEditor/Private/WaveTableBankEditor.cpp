// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableBankEditor.h"

#include "Curves/CurveFloat.h"
#include "CurveEditor.h"
#include "DetailsViewArgs.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "ICurveEditorBounds.h"
#include "IDetailsView.h"
#include "ICurveEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SCurveEditorPanel.h"
#include "WaveTableBank.h"
#include "WaveTableCurveEditorViewStacked.h"
#include "Widgets/Docking/SDockTab.h"


#define LOCTEXT_NAMESPACE "WaveTableEditor"


namespace WaveTable::Editor
{
	namespace BankEditorPrivate
	{
		// Two seconds of samples at 48kHz
		constexpr int32 MaxLiveEditSamples = 48000 * 2;
		constexpr double MinGutter = -0.05;
		constexpr double MaxGutter = 0.05;

		int32 GetDrawPointCount(EWaveTableSamplingMode InMode, int32 InSampleRate, EWaveTableResolution InResolution, const FWaveTableTransform& InTransform)
		{
			auto ResolutionToSampleCount = [](EWaveTableResolution InResolution) { return 1 << static_cast<int32>(InResolution); };
			float NumSamples = 0;
			if (InMode == EWaveTableSamplingMode::FixedResolution)
			{
				EWaveTableResolution DrawPointRes = EWaveTableResolution::None;
				switch (InResolution)
				{
					case EWaveTableResolution::None:
					{
						DrawPointRes = EWaveTableResolution::Res_512;
					}
					break;

					case EWaveTableResolution::Maximum:
					{
						DrawPointRes = EWaveTableResolution::Res_4096;
					}
					break;

					default:
					{
						DrawPointRes = InResolution;
					}
					break;
				};

				NumSamples = ResolutionToSampleCount(DrawPointRes);
			}
			else
			{
				if (InTransform.Curve == EWaveTableCurve::File)
				{
					NumSamples = InTransform.GetTableData().GetNumSamples();
				}
				else
				{
					NumSamples = static_cast<int32>(InTransform.GetDuration() * InSampleRate);
				}
				NumSamples = FMath::Clamp(NumSamples, 2, ResolutionToSampleCount(EWaveTableResolution::Res_Max));
			}

			return NumSamples;
		}
	} // namespace BankEditorPrivate

	const FName FBankEditorBase::AppIdentifier(TEXT("WaveTableEditorApp"));
	const FName FBankEditorBase::CurveTabId(TEXT("WaveTableEditor_Curves"));
	const FName FBankEditorBase::PropertiesTabId(TEXT("WaveTableEditor_Properties"));

	FBankEditorBase::FBankEditorBase()
	{
	}

	void FBankEditorBase::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_WaveTableEditor", "WaveTable Editor"));

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FBankEditorBase::SpawnTab_Properties))
			.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

		FSlateIcon CurveIcon(FAppStyle::GetAppStyleSetName(), "WaveTableEditor.Tabs.Properties");
		InTabManager->RegisterTabSpawner(CurveTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args) { return SpawnTab_OutputCurve(Args); }))
			.SetDisplayName(LOCTEXT("TransformCurvesTab", "Transform Curves"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(CurveIcon);
	}

	void FBankEditorBase::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		InTabManager->UnregisterTabSpawner(PropertiesTabId);
		InTabManager->UnregisterTabSpawner(CurveTabId);
	}

	void FBankEditorBase::Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* InParentObject)
	{
		check(InParentObject);

		CurveEditor = MakeShared<FCurveEditor>();
		FCurveEditorInitParams InitParams;
		CurveEditor->InitCurveEditor(InitParams);
		CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

		TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FStaticCurveEditorBounds>();
		EditorBounds->SetInputBounds(BankEditorPrivate::MinGutter, 1.0 + BankEditorPrivate::MaxGutter);
		CurveEditor->SetBounds(MoveTemp(EditorBounds));

		CurvePanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef());

		// Support undo/redo
		InParentObject->SetFlags(RF_Transactional);
		GEditor->RegisterForUndo(this);

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;
		Args.NotifyHook = this;

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertiesView = PropertyModule.CreateDetailView(Args);
		PropertiesView->SetObject(InParentObject);

		TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_WaveTableEditor_Layout_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.225f)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.775f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.33f)
						->AddTab(CurveTabId, ETabState::OpenedTab)
					)
				)
			)
		);

		const bool bCreateDefaultStandaloneMenu = true;
		const bool bCreateDefaultToolbar = true;
		const bool bToolbarFocusable = false;
		const bool bUseSmallIcons = true;
		FAssetEditorToolkit::InitAssetEditor(
			Mode,
			InitToolkitHost,
			AppIdentifier,
			StandaloneDefaultLayout,
			bCreateDefaultStandaloneMenu,
			bCreateDefaultToolbar,
			InParentObject,
			bToolbarFocusable,
			bUseSmallIcons);

		AddToolbarExtender(CurvePanel->GetToolbarExtender());

		if (CurveEditor.IsValid())
		{
			RegenerateMenusAndToolbars();
		}
	}

	void FBankEditorBase::ClearExpressionCurve(int32 InTransformIndex)
	{
		if (CurveData.IsValidIndex(InTransformIndex))
		{
			CurveData[InTransformIndex].ExpressionCurve.Reset();
		}
	}

	void FBankEditorBase::GenerateExpressionCurve(WaveTable::Editor::FBankEditorBase::FCurveData& OutCurveData, int32 InTransformIndex, bool bInIsUnset)
	{
		TSharedPtr<FRichCurve>& Curve = OutCurveData.ExpressionCurve;
		if (!Curve.IsValid())
		{
			Curve = MakeShared<FRichCurve>();
		}

		const bool bIsBipolar = GetBankIsBipolar();

		if (!GetIsPropertyEditorDisabled())
		{
			if (const FWaveTableTransform* Transform = GetTransform(InTransformIndex))
			{
				const EWaveTableResolution BankResolution = GetBankResolution();
				const EWaveTableSamplingMode BankSamplingMode = GetBankSamplingMode();
				const int32 BankSampleRate = GetBankSampleRate();
				const int32 DrawPointCount = BankEditorPrivate::GetDrawPointCount(BankSamplingMode, BankSampleRate, BankResolution, *Transform);

				FWaveTableData KeyData(EWaveTableBitDepth::IEEE_Float);
				KeyData.Zero(DrawPointCount);
				KeyData.SetFinalValue(0.0f);

				// Optimization for really big source files if set to 'File'. Do not recreate WaveTable on the fly
				// for large samples as caching mechanism is too slow for rapid regeneration (i.e. when making
				// interactive edits).
				if (Transform->WaveTableSettings.SourceData.GetNumSamples() < BankEditorPrivate::MaxLiveEditSamples)
				{
					Transform->CreateWaveTable(KeyData, bIsBipolar);
				}
				else
				{
					Transform->CopyToWaveTable(KeyData, bIsBipolar);
				}

				const float Delta = 1.0 / KeyData.GetNumSamples();
				TArrayView<float> KeyTable;
				ensureAlways(KeyData.GetDataView(KeyTable)); // Format set above; should be success & always return float view

				FRichCurve& CurveRef = *Curve.Get();
				TArray<FRichCurveKey> NewKeys;

				float MaxInputValue = 1.0f;
				switch (BankSamplingMode)
				{
					case EWaveTableSamplingMode::FixedResolution:
					{
						for (int32 i = 0; i < KeyTable.Num(); ++i)
						{
							NewKeys.Add({ Delta * i, KeyTable[i] });
						}
					}
					break;

					case EWaveTableSamplingMode::FixedSampleRate:
					{
						MaxInputValue = Transform->GetDuration();
						if (Transform->Curve == EWaveTableCurve::File)
						{
							MaxInputValue = Transform->GetTableData().GetNumSamples() / (float)BankSampleRate;
						}
						for (int32 i = 0; i < KeyTable.Num(); ++i)
						{
							const float Time = (Delta * i) * MaxInputValue;
							NewKeys.Add({ Time, KeyTable[i] });
						}
					}
					break;

					default:
					{
						static_assert(static_cast<int32>(EWaveTableSamplingMode::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableSamplingMode'");
						checkNoEntry();
					}
				}

				// Terminate curves at initial point for wavetables
				// (as they loop) and at the final computed location
				// for non-wavetables (as they are envelopes).
				if (BankResolution == EWaveTableResolution::None)
				{
					float TermPoint = 1.0f;
					Transform->Apply(TermPoint, bIsBipolar);
					NewKeys.Add({ MaxInputValue, TermPoint });
				}
				else
				{
					NewKeys.Add({ MaxInputValue, KeyTable[0] });
				}

				CurveRef.SetKeys(NewKeys);

				const EWaveTableCurveSource Source = bInIsUnset ? EWaveTableCurveSource::Unset : EWaveTableCurveSource::Expression;
				SetCurve(InTransformIndex, *Curve.Get(), Source);
			}
		}
	}

	bool FBankEditorBase::RequiresNewCurve(int32 InTransformIndex, const FRichCurve& InRichCurve) const
	{
		const FCurveModelID CurveModelID = CurveData[InTransformIndex].ModelID;
		const TUniquePtr<FCurveModel>* CurveModel = CurveEditor->GetCurves().Find(CurveModelID);
		if (!CurveModel || !CurveModel->IsValid())
		{
			return true;
		}

		FWaveTableCurveModel* PatchCurveModel = static_cast<FWaveTableCurveModel*>(CurveModel->Get());
		check(PatchCurveModel);
		if (&PatchCurveModel->GetRichCurve() != &InRichCurve)
		{
			return true;
		}

		return false;
	}

	void FBankEditorBase::SetCurve(int32 InTransformIndex, FRichCurve& InRichCurve, EWaveTableCurveSource InSource)
	{
		check(CurveEditor.IsValid());

		if (!ensure(CurveData.IsValidIndex(InTransformIndex)))
		{
			return;
		}

		FWaveTableTransform* Transform = GetTransform(InTransformIndex);
		if (!ensure(Transform))
		{
			return;
		}

		FCurveData& CurveDataEntry = CurveData[InTransformIndex];

		const bool bIsBipolar = GetBankIsBipolar();
		const bool bRequiresNewCurve = RequiresNewCurve(InTransformIndex, InRichCurve);
		const EWaveTableSamplingMode SamplingMode = GetBankSamplingMode();
		if (bRequiresNewCurve)
		{
			TUniquePtr<FWaveTableCurveModel> NewCurve = ConstructCurveModel(InRichCurve, GetEditingObject(), InSource);
			NewCurve->Refresh(*Transform, InTransformIndex, bIsBipolar, SamplingMode);
			CurveDataEntry.ModelID = CurveEditor->AddCurve(MoveTemp(NewCurve));
			CurveEditor->PinCurve(CurveDataEntry.ModelID);
		}
		else
		{
			const TUniquePtr<FCurveModel>& CurveModel = CurveEditor->GetCurves().FindChecked(CurveDataEntry.ModelID);
			check(CurveModel.Get());
			static_cast<FWaveTableCurveModel*>(CurveModel.Get())->Refresh(*Transform, InTransformIndex, bIsBipolar, SamplingMode);
		}
	}

	FName FBankEditorBase::GetToolkitFName() const
	{
		return FName("WaveTableEditor");
	}

	FText FBankEditorBase::GetBaseToolkitName() const
	{
		return LOCTEXT( "AppLabel", "WaveTable Editor" );
	}

	FString FBankEditorBase::GetWorldCentricTabPrefix() const
	{
		return LOCTEXT("WorldCentricTabPrefix", "WaveTable ").ToString();
	}

	FLinearColor FBankEditorBase::GetWorldCentricTabColorScale() const
	{
		return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
	}

	EOrientation FBankEditorBase::GetSnapLabelOrientation() const
	{
		return FMultiBoxSettings::UseSmallToolBarIcons.Get()
			? EOrientation::Orient_Horizontal
			: EOrientation::Orient_Vertical;
	}

	void FBankEditorBase::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
	{
		const bool bIsInteractive = PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive;
		if (bIsInteractive)
		{
			RegenerateFileCurves();
		}
		else
		{
			RefreshCurves();

			if (CurveEditor.IsValid())
			{
				double MinBound = 0.0;
				double MaxBound = 0.0;
				for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : CurveEditor->GetCurves())
				{
					double MinTime = 0.0;
					double MaxTime = 1.0;
					Pair.Value->GetTimeRange(MinTime, MaxTime);
					MinBound = FMath::Min(MinBound, MinTime);
					MaxBound = FMath::Max(MaxBound, MaxTime);
				}
				CurveEditor->GetBounds().SetInputBounds(MinBound - BankEditorPrivate::MinGutter, MaxBound + BankEditorPrivate::MaxGutter);
			}
		}
	}

	TSharedRef<SDockTab> FBankEditorBase::SpawnTab_Properties(const FSpawnTabArgs& Args)
	{
		check(Args.GetTabId() == PropertiesTabId);

		return SNew(SDockTab)
			.Label(LOCTEXT("SoundWaveTableDetailsTitle", "Details"))
			[
				PropertiesView.ToSharedRef()
			];
	}

	TSharedRef<SDockTab> FBankEditorBase::SpawnTab_OutputCurve(const FSpawnTabArgs& Args)
	{
		RefreshCurves();
		CurveEditor->ZoomToFit();

		TSharedRef<SDockTab> NewDockTab = SNew(SDockTab)
			.Label(FText::Format(LOCTEXT("WaveTableFilterTitle", "Filter Transform Curve: {0}"), FText::FromString(GetEditingObject()->GetName())))
			.TabColorScale(GetTabColorScale())
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					CurvePanel.ToSharedRef()
				]
			];

			return NewDockTab;
	}

	void FBankEditorBase::PostUndo(bool bSuccess)
	{
		if (bSuccess)
		{
			RefreshCurves();
		}
	}

	void FBankEditorBase::ResetCurves()
	{
		check(CurveEditor.IsValid());

		CurveEditor->RemoveAllCurves();
		CurveData.Reset();
		CurveData.AddDefaulted(GetNumTransforms());
	}

	void FBankEditorBase::InitCurves()
	{
		for (int32 i = 0; i < GetNumTransforms(); ++i)
		{
			const FWaveTableTransform* Transform = GetTransform(i);
			if (!ensure(Transform))
			{
				continue;
			}

			switch (Transform->Curve)
			{
				case EWaveTableCurve::Exp:
				case EWaveTableCurve::Exp_Inverse:
				case EWaveTableCurve::Linear:
				case EWaveTableCurve::Linear_Inv:
				case EWaveTableCurve::Log:
				case EWaveTableCurve::SCurve:
				case EWaveTableCurve::Sin:
				case EWaveTableCurve::Sin_Full:
				case EWaveTableCurve::File:
				{
					if (RequiresNewCurve(i, *CurveData[i].ExpressionCurve.Get()))
					{
						ResetCurves();
					}
				}
				break;

				case EWaveTableCurve::Shared:
				{
					if (const UCurveFloat* SharedCurve = Transform->CurveShared)
					{
						if (RequiresNewCurve(i, SharedCurve->FloatCurve))
						{
							ResetCurves();
						}
					}
					else if (RequiresNewCurve(i, *CurveData[i].ExpressionCurve.Get()))
					{
						ResetCurves();
					}
				}
				break;

				case EWaveTableCurve::Custom:
				{
					if (RequiresNewCurve(i, Transform->CurveCustom))
					{
						ResetCurves();
					}
				}
				break;

				default:
				{
					static_assert(static_cast<int32>(EWaveTableCurve::Count) == 11, "Possible missing case coverage for output curve.");
				}
				break;
			}
		}
	}

	void FBankEditorBase::RegenerateFileCurves()
	{
		for (int32 i = 0; i < GetNumTransforms(); ++i)
		{
			if (CurveData.IsValidIndex(i))
			{
				FCurveData& CurveDataEntry = CurveData[i];

				if (FWaveTableTransform* Transform = GetTransform(i))
				{
					if (Transform->Curve == EWaveTableCurve::File)
					{
						GenerateExpressionCurve(CurveDataEntry, i);
					}
				}
			}
		}
	}

	void FBankEditorBase::RefreshCurves()
	{
		check(CurveEditor.IsValid());

		for (const FCurveData& CurveDataEntry : CurveData)
		{
			CurveEditor->UnpinCurve(CurveDataEntry.ModelID);
		}

		if (GetIsPropertyEditorDisabled())
		{
			ResetCurves();
			return;
		}

		const int32 NumCurves = GetNumTransforms();
		if (NumCurves == CurveData.Num())
		{
			InitCurves();
		}
		else
		{
			ResetCurves();
		}

		for (int32 i = 0; i < GetNumTransforms(); ++i)
		{
			if (!CurveData.IsValidIndex(i))
			{
				continue;
			}
			FCurveData& CurveDataEntry = CurveData[i];

			FWaveTableTransform* Transform = GetTransform(i);
			if (!Transform)
			{
				continue;
			}

			switch (Transform->Curve)
			{
				case EWaveTableCurve::Exp:
				case EWaveTableCurve::Exp_Inverse:
				case EWaveTableCurve::Linear:
				case EWaveTableCurve::Linear_Inv:
				case EWaveTableCurve::Log:
				case EWaveTableCurve::SCurve:
				case EWaveTableCurve::Sin:
				case EWaveTableCurve::Sin_Full:
				case EWaveTableCurve::File:
				{
					GenerateExpressionCurve(CurveDataEntry, i);
				}
				break;

				case EWaveTableCurve::Shared:
				{
					if (UCurveFloat* SharedCurve = Transform->CurveShared)
					{
						ClearExpressionCurve(i);
						SetCurve(i, SharedCurve->FloatCurve, EWaveTableCurveSource::Shared);
					}
					else
					{
						// Builds a dummy expression that just maps input to output in case
						// where asset isn't selected and leave source as unset
						GenerateExpressionCurve(CurveDataEntry, i, true /* bIsUnset */);
					}
				}
				break;

				case EWaveTableCurve::Custom:
				{
					FRichCurve& CustomCurve = Transform->CurveCustom;
					TrimKeys(CustomCurve);
					ClearExpressionCurve(i);
					SetCurve(i, CustomCurve, EWaveTableCurveSource::Custom);
				}
				break;

				default:
				{
					static_assert(static_cast<int32>(EWaveTableCurve::Count) == 11, "Possible missing case coverage for output curve.");
				}
				break;
			}
		}

		// Collect and remove stale curves from editor
		TArray<FCurveModelID> ToRemove;
		TSet<FCurveModelID> ActiveModelIDs;
		for (const FCurveData& CurveDataEntry : CurveData)
		{
			ActiveModelIDs.Add(CurveDataEntry.ModelID);
		}

		// Remove all dead curves
		{
			const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& Curves = CurveEditor->GetCurves();
			for (const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& Pair : Curves)
			{
				if (!ActiveModelIDs.Contains(Pair.Key))
				{
					ToRemove.Add(Pair.Key);
				}
			}

			for (FCurveModelID ModelID : ToRemove)
			{
				CurveEditor->RemoveCurve(ModelID);
			}
		}

		// Re-pin all curves
		for (const FCurveData& CurveDataEntry : CurveData)
		{
			CurveEditor->PinCurve(CurveDataEntry.ModelID);
		}
	}

	void FBankEditorBase::PostRedo(bool bSuccess)
	{
		if (bSuccess)
		{
			RefreshCurves();
		}
	}

	void FBankEditorBase::TrimKeys(FRichCurve& OutCurve)
	{
		while (OutCurve.GetNumKeys() > 0 && 0.0f > OutCurve.GetFirstKey().Time)
		{
			OutCurve.DeleteKey(OutCurve.GetFirstKeyHandle());
		}

		while (OutCurve.GetNumKeys() > 0 && 1.0f < OutCurve.GetLastKey().Time)
		{
			OutCurve.DeleteKey(OutCurve.GetLastKeyHandle());
		}
	}

	TUniquePtr<WaveTable::Editor::FWaveTableCurveModel> FBankEditor::ConstructCurveModel(FRichCurve& InRichCurve, UObject* InParentObject, EWaveTableCurveSource InSource)
	{
		using namespace WaveTable::Editor;
		return MakeUnique<FWaveTableCurveModel>(InRichCurve, GetEditingObject(), InSource);
	}

	EWaveTableResolution FBankEditor::GetBankResolution() const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			return Bank->Resolution;
		}

		return EWaveTableResolution::None;
	}

	EWaveTableSamplingMode FBankEditor::GetBankSamplingMode() const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			return Bank->SampleMode;
		}

		return EWaveTableSamplingMode::FixedResolution;
	}

	int32 FBankEditor::GetBankSampleRate() const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			return Bank->SampleRate;
		}

		return 48000;
	}

	bool FBankEditor::GetBankIsBipolar() const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			return Bank->bBipolar;
		}

		return false;
	}

	int32 FBankEditor::GetNumTransforms() const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			return Bank->GetEntries().Num();
		}

		return 0;
	}

	FWaveTableTransform* FBankEditor::GetTransform(int32 InIndex) const
	{
		if (UWaveTableBank* Bank = Cast<UWaveTableBank>(GetEditingObject()))
		{
			if (Bank->GetEntries().IsValidIndex(InIndex))
			{
				return &Bank->GetEntries()[InIndex].Transform;
			}
		}

		return nullptr;
	}
} // namespace WaveTable::Editor
#undef LOCTEXT_NAMESPACE
