// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableBankFactory.h"
#include "WaveTableBank.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTableBankFactory)


UWaveTableBankFactory::UWaveTableBankFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWaveTableBank::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UWaveTableBankFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UWaveTableBank>(InParent, Name, Flags);
}
