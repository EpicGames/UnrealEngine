// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveTableCurveEditorViewStacked.h"

#include "Curves/CurveFloat.h"
#include "CurveEditor.h"
#include "Fonts/FontMeasure.h"
#include "Rendering/SlateRenderer.h"
#include "SCurveEditorPanel.h"
#include "Styling/CoreStyle.h"
#include "WaveTableTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveTableCurveEditorViewStacked)

#define LOCTEXT_NAMESPACE "WaveTableEditor"


namespace WaveTable
{
	namespace Editor
	{
		FWaveTableCurveModel::FWaveTableCurveModel(FRichCurve& InRichCurve, UObject* InOwner, EWaveTableCurveSource InSource)
			: FRichCurveEditorModelRaw(&InRichCurve, InOwner)
			, ParentObject(InOwner)
			, Source(InSource)
		{
			check(InOwner);
		}

		void FWaveTableCurveModel::Refresh(const FWaveTableTransform& InTransform, int32 InCurveIndex, bool bInIsBipolar, EWaveTableSamplingMode InSamplingMode)
		{
			// Must be set prior to remainder of refresh to avoid
			// child class implementation acting on incorrect cached index
			// (Editor may re-purpose models resulting in index change).
			CurveIndex = InCurveIndex;

			CurveDuration = InTransform.GetDuration();
			Color = GetCurveColor();
			SamplingMode = InSamplingMode;

			FText OutputAxisName;
			RefreshCurveDescriptorText(InTransform, ShortDisplayName, InputAxisName, OutputAxisName);
			AxesDescriptor = FText::Format(LOCTEXT("WaveTableCurveDisplayTitle_AxesFormat", "X = {0}, Y = {1}"), InputAxisName, OutputAxisName);

			IntentionName = TEXT("AudioControlValue");
			SupportedViews = GetViewId();

			bKeyDrawEnabled = false;

			bIsBipolar = bInIsBipolar;

			NumSamples = InTransform.GetTableData().GetNumSamples();

			if (InTransform.Curve == EWaveTableCurve::File)
			{
				FadeInRatio = InTransform.WaveTableSettings.FadeIn;
				FadeOutRatio = InTransform.WaveTableSettings.FadeOut;
			}
			else
			{
				FadeInRatio = 0.0f;
				FadeOutRatio = 0.0f;
			}

			const bool bIsDisabled = GetPropertyEditorDisabled();
			if (bIsDisabled)
			{
				LongDisplayName = FText::Format(LOCTEXT("WaveTableCurveDisplay_BypassedNameFormat", "{0} ({1})"), ShortDisplayName, GetPropertyEditorDisabledText());
			}
			else
			{
				LongDisplayName = FText::Format(LOCTEXT("WaveTableCurveDisplayTitle_NameFormat", "{0}: {1}"), FText::AsNumber(CurveIndex), ShortDisplayName);

				if (Source == EWaveTableCurveSource::Custom)
				{
					bKeyDrawEnabled = true;
				}
				else if (Source == EWaveTableCurveSource::Shared)
				{
					bKeyDrawEnabled = true;

					if (const UCurveFloat* SharedCurve = InTransform.CurveShared)
					{
						const FText CurveSourceText = FText::FromString(SharedCurve->GetName());
						LongDisplayName = FText::Format(LOCTEXT("WaveTableCurveDisplayTitle_NameSharedFormat", "{0}: {1}, Shared Curve '{2}'"), FText::AsNumber(CurveIndex), ShortDisplayName, CurveSourceText);
					}
				}
			}
		}

		FLinearColor FWaveTableCurveModel::GetColor() const
		{
			return !GetPropertyEditorDisabled() && (Source == EWaveTableCurveSource::Custom)
				? Color
				: Color.Desaturate(0.35f);
		}

		void FWaveTableCurveModel::GetTimeRange(double& MinValue, double& MaxValue) const
		{
			if (Source == EWaveTableCurveSource::Shared)
			{
				MinValue = 0.0;
				MaxValue = 1.0;
			}
			else
			{
				FRichCurveEditorModelRaw::GetTimeRange(MinValue, MaxValue);
				if (FMath::IsNearlyEqual(MinValue, MaxValue))
				{
					MinValue -= 0.01;
					MaxValue += 0.01;
				}
			}
		}

		void FWaveTableCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
		{
			if (Source == EWaveTableCurveSource::Shared || (FadeInRatio > 0.0f || FadeOutRatio > 0.0f))
			{
				MaxValue = 1.0;

				MinValue = bIsBipolar ? -1.0 : 0.0;
			}
			else
			{
				FRichCurveEditorModelRaw::GetValueRange(MinValue, MaxValue);
				if (FMath::IsNearlyEqual(MinValue, MaxValue))
				{
					MinValue -= 0.01;
					MaxValue += 0.01;
				}
			}
		}

		const FText& FWaveTableCurveModel::GetAxesDescriptor() const
		{
			return AxesDescriptor;
		}

		EWaveTableCurveSource FWaveTableCurveModel::GetSource() const
		{
			return Source;
		}

		const UObject* FWaveTableCurveModel::GetParentObject() const
		{
			return ParentObject.Get();
		}

		bool FWaveTableCurveModel::IsReadOnly() const
		{
			return Source != EWaveTableCurveSource::Custom;
		}

		ECurveEditorViewID FWaveTableCurveModel::WaveTableViewId = ECurveEditorViewID::Invalid;

		void FWaveTableCurveModel::RefreshCurveDescriptorText(const FWaveTableTransform& InTransform, FText& OutShortDisplayName, FText& OutInputAxisName, FText& OutOutputAxisName)
		{
			UEnum* Enum = StaticEnum<EWaveTableCurve>();
			check(Enum);

			Enum->GetDisplayValueAsText(InTransform.Curve, OutShortDisplayName);

			switch (SamplingMode)
			{
				case EWaveTableSamplingMode::FixedResolution:
				{
					OutInputAxisName = LOCTEXT("WaveTableCurveModel_DefaultFixedResolutionInputAxisName", "Unit Phase (Table Index)");
				}
				break;

				case EWaveTableSamplingMode::FixedSampleRate:
				{
					OutInputAxisName = LOCTEXT("WaveTableCurveModel_DefaultFixedSampleRateInputAxisName", "Time");
				}
				break;

				default:
				{
					static_assert(static_cast<int32>(EWaveTableSamplingMode::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableSamplingMode'");
					checkNoEntry();
				}
			};

			OutOutputAxisName = LOCTEXT("WaveTableCurveModel_DefaultOutputAxisName", "Amplitude");
		}

		FColor FWaveTableCurveModel::GetCurveColor() const
		{
			return FColor { 232, 122, 0, 255 };
		}
		
		bool FWaveTableCurveModel::GetPropertyEditorDisabled() const
		{
			return false;
		}

		FText FWaveTableCurveModel::GetPropertyEditorDisabledText() const
		{
			return LOCTEXT("WaveTableCurveModel_Disabled", "Disabled");
		}

		void SViewStacked::Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor)
		{
			SCurveEditorViewStacked::Construct(InArgs, InCurveEditor);

			StackedHeight = 300.0f;
			StackedPadding = 60.0f;
		}

		void SViewStacked::PaintView(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (CurveEditor)
			{
				const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

				DrawBackground(AllottedGeometry, OutDrawElements, BaseLayerId, DrawEffects);
				DrawViewGrids(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
				DrawLabels(AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, DrawEffects);
				DrawCurves(CurveEditor.ToSharedRef(), AllottedGeometry, MyCullingRect, OutDrawElements, BaseLayerId, InWidgetStyle, DrawEffects);
			}
		}

		void SViewStacked::DrawLabels(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (!CurveEditor)
			{
				return;
			}

			const double ValuePerPixel = 1.0 / StackedHeight;
			const double ValueSpacePadding = StackedPadding * ValuePerPixel;

			const FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("CurveEd.LabelFont");
			const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
			const FCurveEditorScreenSpace ViewSpace = GetViewSpace();

			// Draw the curve labels for each view
			for (const TPair<FCurveModelID, SCurveEditorView::FCurveInfo>& Pair : CurveInfoByID)
			{
				FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key);
				if (!ensureAlways(Curve))
				{
					continue;
				}

				const int32  CurveIndexFromBottom = CurveInfoByID.Num() - Pair.Value.CurveIndex - 1;
				const double PaddingToBottomOfView = (CurveIndexFromBottom + 1) * ValueSpacePadding;

				const double PixelBottom = ViewSpace.ValueToScreen(CurveIndexFromBottom + PaddingToBottomOfView);
				const double PixelTop = ViewSpace.ValueToScreen(CurveIndexFromBottom + PaddingToBottomOfView + 1.0);

				if (!FSlateRect::DoRectanglesIntersect(MyCullingRect, TransformRect(AllottedGeometry.GetAccumulatedLayoutTransform(), FSlateRect(0, PixelTop, LocalSize.X, PixelBottom))))
				{
					continue;
				}

				const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

				// Render label
				FText Label = Curve->GetLongDisplayName();

				static const float LabelOffsetX = 15.0f;
				static const float LabelOffsetY = 35.0f;
				FVector2D LabelPosition(LabelOffsetX, PixelTop - LabelOffsetY);
				FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(LabelPosition));
				const FVector2D LabelSize = FontMeasure->Measure(Label, FontInfo);

				const uint32 LabelLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::GridLabels;
				FSlateDrawElement::MakeText(OutDrawElements, LabelLayerId + 1, LabelGeometry, Label, FontInfo, DrawEffects, Curve->GetColor());

				// Render axes descriptor
				FText Descriptor = static_cast<FWaveTableCurveModel*>(Curve)->GetAxesDescriptor();

				const FVector2D DescriptorSize = FontMeasure->Measure(Descriptor, FontInfo);

				static const float LabelBufferX = 20.0f; // Keeps label and axes descriptor visually separated
				static const float GutterBufferX = 20.0f; // Accounts for potential scroll bar
				const float ViewWidth = ViewSpace.GetPhysicalWidth();
				const float FloatingDescriptorX = ViewWidth - DescriptorSize.X;
				LabelPosition = FVector2D(FMath::Max(LabelSize.X + LabelBufferX, FloatingDescriptorX - GutterBufferX), PixelTop - LabelOffsetY);
				LabelGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(LabelPosition));

				FSlateDrawElement::MakeText(OutDrawElements, LabelLayerId + 1, LabelGeometry, Descriptor, FontInfo, DrawEffects, Curve->GetColor());
			}
		}

		void SViewStacked::DrawViewGrids(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const
		{
			TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
			if (!CurveEditor)
			{
				return;
			}

			// Rendering info
			const float          Width = AllottedGeometry.GetLocalSize().X;
			const float          Height = AllottedGeometry.GetLocalSize().Y;
			const FSlateBrush*   WhiteBrush = FAppStyle::GetBrush("WhiteBrush");

			FGridDrawInfo DrawInfo(&AllottedGeometry, GetViewSpace(), CurveEditor->GetPanel()->GetGridLineTint(), BaseLayerId);

			TArray<float> MajorGridLinesX, MinorGridLinesX;
			TArray<FText> MajorGridLabelsX;

			GetGridLinesX(CurveEditor.ToSharedRef(), MajorGridLinesX, MinorGridLinesX, &MajorGridLabelsX);

			const double ValuePerPixel = 1.0 / StackedHeight;
			const double ValueSpacePadding = StackedPadding * ValuePerPixel;

			if (CurveInfoByID.IsEmpty())
			{
				return;
			}

			for (auto It = CurveInfoByID.CreateConstIterator(); It; ++It)
			{
				DrawInfo.SetCurveModel(CurveEditor->FindCurve(It.Key()));
				const FCurveModel* CurveModel = DrawInfo.GetCurveModel();
				if (!ensureAlways(CurveModel))
				{
					continue;
				}

				TArray<FText> CurveModelGridLabelsX = MajorGridLabelsX;
				check(MajorGridLinesX.Num() == CurveModelGridLabelsX.Num());

				if (const FWaveTableCurveModel* EditorModel = static_cast<const FWaveTableCurveModel*>(CurveModel))
				{
					for (int32 i = 0; i < CurveModelGridLabelsX.Num(); ++i)
					{
						FText& InputLabel = CurveModelGridLabelsX[i];
						FormatInputLabel(*EditorModel, DrawInfo.LabelFormat, InputLabel);
					}
				}

				const int32 Index = CurveInfoByID.Num() - It->Value.CurveIndex - 1;
				double Padding = (Index + 1) * ValueSpacePadding;
				DrawInfo.SetLowerValue(Index + Padding);

				FSlateRect BoundsRect(0, DrawInfo.GetPixelTop(), Width, DrawInfo.GetPixelBottom());
				if (!FSlateRect::DoRectanglesIntersect(MyCullingRect, TransformRect(AllottedGeometry.GetAccumulatedLayoutTransform(), BoundsRect)))
				{
					continue;
				}

				// Tint the views based on their curve color
				{
					FLinearColor CurveColorTint = DrawInfo.GetCurveModel()->GetColor().CopyWithNewOpacity(0.05f);
					const FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(
						FVector2D(Width, StackedHeight),
						FSlateLayoutTransform(FVector2D(0.f, DrawInfo.GetPixelTop()))
					);

					const int32 GridOverlayLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridOverlays;
					FSlateDrawElement::MakeBox(OutDrawElements, GridOverlayLayerId, BoxGeometry, WhiteBrush, DrawEffects, CurveColorTint);
				}

				// Horizontal grid lines
				DrawInfo.LinePoints[0].X = 0.0;
				DrawInfo.LinePoints[1].X = Width;

				double DefaultValueHeight = -1.0;
				if (const FWaveTableCurveModel* WaveTableCurveModel = static_cast<const FWaveTableCurveModel*>(CurveModel))
				{
					if (WaveTableCurveModel->ShowDefaultDashedLine())
					{
						double ValueMin = 0.0;
						double ValueMax = 0.0;
						WaveTableCurveModel->GetValueRange(ValueMin, ValueMax);
						double DefaultValue = WaveTableCurveModel->GetDefaultDashedLineValue(); 
						DefaultValueHeight = FMath::GetMappedRangeValueClamped(TRange<double>(ValueMin, ValueMax), TRange<double>(0.0, 1.0), DefaultValue);
					}
				}

				// Dashed line on the default value
				if (const FWaveTableCurveModel* WaveTableCurveModel = static_cast<const FWaveTableCurveModel*>(CurveModel))
				{
					if (WaveTableCurveModel->ShowDefaultDashedLine())
					{	
						DrawViewGridLineX(OutDrawElements, DrawInfo, DrawEffects, DefaultValueHeight  /* OffsetAlpha */, true  /* bIsMajor */, true /* bDashedLine */);
					}
				}
				
				TArray<double>OffsetAlphas = { 1.0, 0.75, 0.5, 0.25, 0.0 };
				for (int OffsetAlphaIndex = 0; OffsetAlphaIndex < OffsetAlphas.Num(); ++OffsetAlphaIndex)
				{
					// Skip dashed line
					double OffsetAlpha = OffsetAlphas[OffsetAlphaIndex];
					if (FMath::IsNearlyEqual(OffsetAlpha, DefaultValueHeight))
					{
						continue;
					}
					// Even indices are major lines
					bool bIsMajor = !static_cast<bool>(OffsetAlphaIndex % 2);
					DrawViewGridLineX(OutDrawElements, DrawInfo, DrawEffects, OffsetAlphas[OffsetAlphaIndex], bIsMajor, false /* bDashedLine */);
				}

				// Vertical grid lines
				{
					DrawInfo.LinePoints[0].Y = DrawInfo.GetPixelTop();
					DrawInfo.LinePoints[1].Y = DrawInfo.GetPixelBottom();

					// Draw major vertical grid lines
					for (int32 i = 0; i < MajorGridLinesX.Num(); ++i)
					{
						const float VerticalLine = FMath::RoundToFloat(MajorGridLinesX[i]);
						const FText& Label = CurveModelGridLabelsX[i];

						DrawViewGridLineY(VerticalLine, OutDrawElements, DrawInfo, DrawEffects, &Label, true /* bIsMajor */);
					}

					// Now draw the minor vertical lines which are drawn with a lighter color.
					for (float VerticalLine : MinorGridLinesX)
					{
						VerticalLine = FMath::RoundToFloat(VerticalLine);
						DrawViewGridLineY(VerticalLine, OutDrawElements, DrawInfo, DrawEffects, nullptr /* Label */, false /* bIsMajor */);
					}
				}

				if (const FWaveTableCurveModel* EditorModel = static_cast<const FWaveTableCurveModel*>(CurveModel))
				{
					static const FLinearColor FadeColor = FLinearColor::White.CopyWithNewOpacity(0.75f);

					const int32 GridOverlayLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridOverlays;
					const int32 FadeLayerId = BaseLayerId + CurveViewConstants::ELayerOffset::Curves;
					const FCurveModelID ModelID = It.Key();
					const bool bIsBipolar = EditorModel->GetIsBipolar();

					enum ERegionType : uint8 { FadeIn, FadeOut, Invalid };
					auto DrawRegion = [&](double StartPosX, double EndPosX, ERegionType RegionType, const FSlateBrush* BoxBrush)
					{
						const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(It.Key());
						FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(ModelID);

						StartPosX = CurveSpace.SecondsToScreen(StartPosX);
						EndPosX = CurveSpace.SecondsToScreen(EndPosX);

						DrawInfo.LinePoints[0].X = (double)StartPosX;
						DrawInfo.LinePoints[1].X = (double)EndPosX;

						double Width = 0.0;
						bool bDrawBox = true;
						switch (RegionType)
						{
							case ERegionType::FadeIn:
							case ERegionType::FadeOut:
							{
								Width = FMath::Abs(DrawInfo.LinePoints[1].X - DrawInfo.LinePoints[0].X);
								DrawInfo.LinePoints[0].Y = CurveSpace.ValueToScreen(0.0f);
								DrawInfo.LinePoints[1].Y = CurveSpace.ValueToScreen(1.0f);

								FSlateDrawElement::MakeLines(OutDrawElements, FadeLayerId, DrawInfo.PaintGeometry, DrawInfo.LinePoints, DrawEffects, FadeColor, false);

								if (bIsBipolar)
								{
									DrawInfo.LinePoints[1].Y = CurveSpace.ValueToScreen(-1.0f);
									FSlateDrawElement::MakeLines(OutDrawElements, FadeLayerId, DrawInfo.PaintGeometry, DrawInfo.LinePoints, DrawEffects, FadeColor, false);
								}
							}
							break;

							case ERegionType::Invalid:
							default:
							{
								Width = DrawInfo.LinePoints[1].X - DrawInfo.LinePoints[0].X;
								bDrawBox = Width > 0.0;
							}
							break;
						}

						if (bDrawBox)
						{
							const double BoxStart = RegionType == ERegionType::FadeIn || RegionType == ERegionType::Invalid ? StartPosX : StartPosX - Width;
							const FPaintGeometry BoxGeometry = DrawInfo.AllottedGeometry->ToPaintGeometry(
								FVector2D(Width, StackedHeight),
								FSlateLayoutTransform(FVector2D(BoxStart, DrawInfo.GetPixelTop()))
							);

							FSlateDrawElement::MakeBox(OutDrawElements, GridOverlayLayerId, BoxGeometry, BoxBrush, DrawEffects, FadeColor.CopyWithNewOpacity(0.15f));
						}
					};

					const float Duration = EditorModel->GetSamplingMode() == EWaveTableSamplingMode::FixedResolution ? 1.0f : EditorModel->GetCurveDuration();
					const float FadeInRatio = EditorModel->GetFadeInRatio();
					if (FadeInRatio > 0.0f)
					{
						DrawRegion(0.0, (double)(Duration * FadeInRatio), ERegionType::FadeIn, WhiteBrush);
					}

					const float FadeOutRatio = EditorModel->GetFadeOutRatio();
					if (FadeOutRatio > 0.0f)
					{
						DrawRegion((double)Duration, (double)(Duration * (1.0f - FadeOutRatio)), ERegionType::FadeOut, WhiteBrush);
					}

					if (EditorModel->GetSource() == EWaveTableCurveSource::Shared || EditorModel->GetSource() == EWaveTableCurveSource::Custom)
					{
						const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(It.Key());
						FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(ModelID);

						const FSlateBrush* BlackBrush = FAppStyle::GetBrush("BlackBrush");
						DrawRegion(CurveSpace.GetInputMin(), 0.0, ERegionType::Invalid, BlackBrush);
						DrawRegion(1.0, CurveSpace.GetInputMax(), ERegionType::Invalid, BlackBrush);
					}
				}
			}
		}

		void SViewStacked::DrawViewGridLineX(FSlateWindowElementList& OutDrawElements, FGridDrawInfo& DrawInfo, ESlateDrawEffect DrawEffects, double OffsetAlpha, bool bIsMajor, bool bDashedLine) const
		{
			double ValueMin = 0.0;
			double ValueMax = 0.0;
			DrawInfo.GetCurveModel()->GetValueRange(ValueMin, ValueMax);

			const double LowerValue = DrawInfo.GetLowerValue();
			const double PixelBottom = DrawInfo.GetPixelBottom();
			const double PixelTop = DrawInfo.GetPixelTop();

			
			const uint32 LineLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridLines;

			FLinearColor Color = bIsMajor ? DrawInfo.GetMajorGridColor() : DrawInfo.GetMinorGridColor();

			DrawInfo.LinePoints[0].Y = DrawInfo.LinePoints[1].Y = DrawInfo.ScreenSpace.ValueToScreen(LowerValue + OffsetAlpha);

			if (bDashedLine)
			{
				// There is no FVector2d version of MakeDashedLines, so convert 
				TArray<FVector2f> FloatPoints;
				Algo::Transform(DrawInfo.LinePoints, FloatPoints, [](FVector2d DoubleIn)
					{
						return FVector2f(DoubleIn);
					});
				FSlateDrawElement::MakeDashedLines(OutDrawElements, LineLayerId, DrawInfo.PaintGeometry, MoveTemp(FloatPoints), DrawEffects, Color);
			}
			else
			{
				FSlateDrawElement::MakeLines(OutDrawElements, LineLayerId, DrawInfo.PaintGeometry, DrawInfo.LinePoints, DrawEffects, Color, false);
			}

			// Label
			FText Label = FText::AsNumber(FMath::Lerp(ValueMin, ValueMax, OffsetAlpha), &DrawInfo.LabelFormat);

			if (const FWaveTableCurveModel* CurveModel = static_cast<const FWaveTableCurveModel*>(DrawInfo.GetCurveModel()))
			{
				FormatOutputLabel(*CurveModel, DrawInfo.LabelFormat, Label);
			}

			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("CurveEd.InfoFont");

			const FVector2D LabelSize = FontMeasure->Measure(Label, FontInfo);
			double LabelY = FMath::Lerp(PixelBottom, PixelTop, OffsetAlpha);
	
			// Offset label Y position below line only if the top gridline,
			// otherwise push above
			if (FMath::IsNearlyEqual(OffsetAlpha, 1.0))
			{
				LabelY += 5.0;
			}
			else
			{
				LabelY -= 15.0;
			}

			const FPaintGeometry LabelGeometry = DrawInfo.AllottedGeometry->ToPaintGeometry(FSlateLayoutTransform(FVector2D(CurveViewConstants::CurveLabelOffsetX, LabelY)));
			const uint32 LabelLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridLabels;

			FSlateDrawElement::MakeText(
				OutDrawElements,
				LabelLayerId,
				LabelGeometry,
				Label,
				FontInfo,
				DrawEffects,
				DrawInfo.GetLabelColor()
			);
		}

		void SViewStacked::DrawViewGridLineY(const float VerticalLine, FSlateWindowElementList& OutDrawElements, FGridDrawInfo& DrawInfo, ESlateDrawEffect DrawEffects, const FText* Label, bool bIsMajor) const
		{
			const float Width = DrawInfo.AllottedGeometry->GetLocalSize().X;
			if (VerticalLine >= 0 || VerticalLine <= FMath::RoundToFloat(Width))
			{
				const FLinearColor LineColor = bIsMajor ? DrawInfo.GetMajorGridColor() : DrawInfo.GetMinorGridColor();

				DrawInfo.LinePoints[0].X = DrawInfo.LinePoints[1].X = VerticalLine;
				const uint32 LineLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridLines;
				FSlateDrawElement::MakeLines(OutDrawElements, LineLayerId, DrawInfo.PaintGeometry, DrawInfo.LinePoints, DrawEffects, LineColor, false);

				if (Label)
				{
					const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("CurveEd.InfoFont");

					const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
					const FVector2D LabelSize = FontMeasure->Measure(*Label, FontInfo);
					const FPaintGeometry LabelGeometry = DrawInfo.AllottedGeometry->ToPaintGeometry(FSlateLayoutTransform(FVector2D(VerticalLine, DrawInfo.LinePoints[0].Y - LabelSize.Y * 1.2f)));
					const uint32 LabelLayerId = DrawInfo.GetBaseLayerId() + CurveViewConstants::ELayerOffset::GridLabels;

					FSlateDrawElement::MakeText(
						OutDrawElements,
						LabelLayerId,
						LabelGeometry,
						*Label,
						FontInfo,
						DrawEffects,
						DrawInfo.GetLabelColor()
					);
				}
			}
		}

		void SViewStacked::FormatInputLabel(const FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const
		{
			switch (EditorModel.GetSamplingMode())
			{
				case EWaveTableSamplingMode::FixedResolution:
				{
					int32 NumSamples = EditorModel.GetNumSamples();
					if (NumSamples != 0)
					{
						const float Value = FCString::Atof(*InOutLabel.ToString());
						if (Value >= 0 && Value <= 1.0f)
						{
							NumSamples = FMath::FloorToInt32(FMath::Abs(NumSamples * Value));
							InOutLabel = FText::Format(LOCTEXT("WaveTableCurveEditor_ResolutionInputLabelFormat", "{0} ({1})"), InOutLabel, FText::AsNumber(NumSamples));
						}
					}
				}
				break;

				case EWaveTableSamplingMode::FixedSampleRate:
				{
					switch (EditorModel.GetSource())
					{
						case EWaveTableCurveSource::Custom:
						case EWaveTableCurveSource::Shared:
						{
							const FText NormalizedLabel = InOutLabel;
							float Value = FCString::Atof(*InOutLabel.ToString()) * EditorModel.GetCurveDuration();
							// Avoids negative 0 display issue
							if (FMath::IsNearlyZero(Value))
							{
								InOutLabel = LOCTEXT("WaveTableCurveEditor_FixedSampleRate_ZeroFormat", "0s");
							}
							else if (Value > 0.0f)
							{
								InOutLabel = FText::Format(LOCTEXT("WaveTableCurveEditor_FixedSampleRate_NormalizedAndDurationInputLabelFormat", "{0} ({1}s)")
									, NormalizedLabel
									, FText::AsNumber(Value, &InLabelFormat));
							}
						}
						break;

						default:
						{
							InOutLabel = FText::Format(LOCTEXT("WaveTableCurveEditor_DurationInputLabelFormat", "{0}s"), InOutLabel);
						}
						break;
					}
				}
				break;

				default:
				{
					static_assert(static_cast<int32>(EWaveTableSamplingMode::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableSamplingMode'");
					checkNoEntry();
				}
			};
		}

		FText SViewStacked::FormatToolTipTime(const FCurveModel& CurveModel, double EvaluatedTime) const
		{
			FNumberFormattingOptions FormatOptions;
			FormatOptions.MaximumFractionalDigits = 3;
			FText Time = FText::AsNumber(EvaluatedTime, &FormatOptions);
			FormatInputLabel(static_cast<const FWaveTableCurveModel&>(CurveModel), FormatOptions, Time);
			return FText::Format(LOCTEXT("WaveTable_PointToolTipTime", "Time: {0}"), Time);
		}


		SViewStacked::FGridDrawInfo::FGridDrawInfo(const FGeometry* InAllottedGeometry, const FCurveEditorScreenSpace& InScreenSpace, FLinearColor InGridColor, int32 InBaseLayerId)
			: AllottedGeometry(InAllottedGeometry)
			, ScreenSpace(InScreenSpace)
			, BaseLayerId(InBaseLayerId)
			, CurveModel(nullptr)
			, LowerValue(0)
			, PixelBottom(0)
			, PixelTop(0)
		{
			check(AllottedGeometry);

			// Pre-allocate an array of line points to draw our vertical lines. Each major grid line
			// will overwrite the X value of both points but leave the Y value untouched so they draw from the bottom to the top.
			LinePoints.Add(FVector2D(0.f, 0.f));
			LinePoints.Add(FVector2D(0.f, 0.f));

			MajorGridColor = InGridColor;
			MinorGridColor = InGridColor.CopyWithNewOpacity(InGridColor.A * 0.5f);

			PaintGeometry = InAllottedGeometry->ToPaintGeometry();

			LabelFormat.SetMaximumFractionalDigits(2);
		}

		void SViewStacked::FGridDrawInfo::SetCurveModel(const FCurveModel* InCurveModel)
		{
			CurveModel = InCurveModel;
		}

		const FCurveModel* SViewStacked::FGridDrawInfo::GetCurveModel() const
		{
			return CurveModel;
		}

		void SViewStacked::FGridDrawInfo::SetLowerValue(double InLowerValue)
		{
			LowerValue = InLowerValue;
			PixelBottom = ScreenSpace.ValueToScreen(InLowerValue);
			PixelTop = ScreenSpace.ValueToScreen(InLowerValue + 1.0);
		}

		int32 SViewStacked::FGridDrawInfo::GetBaseLayerId() const
		{
			return BaseLayerId;
		}

		FLinearColor SViewStacked::FGridDrawInfo::GetLabelColor() const
		{
			check(CurveModel);
			return GetCurveModel()->GetColor().CopyWithNewOpacity(0.7f);
		}

		double SViewStacked::FGridDrawInfo::GetLowerValue() const
		{
			return LowerValue;
		}

		FLinearColor SViewStacked::FGridDrawInfo::GetMajorGridColor() const
		{
			return MajorGridColor;
		}

		FLinearColor SViewStacked::FGridDrawInfo::GetMinorGridColor() const
		{
			return MinorGridColor;
		}

		double SViewStacked::FGridDrawInfo::GetPixelBottom() const
		{
			return PixelBottom;
		}

		double SViewStacked::FGridDrawInfo::GetPixelTop() const
		{
			return PixelTop;
		}
	} // namespace Editor
} // namespace WaveTable

#undef LOCTEXT_NAMESPACE
