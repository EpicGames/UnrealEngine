// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableEditorModule.h"

#include "CurveDrawInfo.h"
#include "ICurveEditorModule.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "WaveTableCurveEditorViewStacked.h"
#include "WaveTableTransformLayout.h"

DEFINE_LOG_CATEGORY(LogWaveTableEditor);


namespace WaveTable::Editor
{
	namespace ModulePrivate
	{
		static const FName AssetToolName { "AssetTools" };
		static const FName CurveEditorName { "CurveEditor" };
		static const FName PropertyEditorName { "PropertyEditor" };

		template <typename T>
		void AddAssetAction(IAssetTools& AssetTools, TArray<TSharedPtr<FAssetTypeActions_Base>>& AssetArray)
		{
			TSharedPtr<T> AssetAction = MakeShared<T>();
			TSharedPtr<FAssetTypeActions_Base> AssetActionBase = StaticCastSharedPtr<FAssetTypeActions_Base>(AssetAction);
			AssetTools.RegisterAssetTypeActions(AssetAction.ToSharedRef());
			AssetArray.Add(AssetActionBase);
		}

		class FSlateStyle final : public FSlateStyleSet
		{
		public:
			FSlateStyle()
				: FSlateStyleSet("WaveTableStyle")
			{
				SetParentStyleName(FAppStyle::GetAppStyleSetName());

				SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/WaveTable/Content/Editor/Slate"));
				SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

				static const FVector2D Icon16 = FVector2D(16.0f, 16.0f);
				static const FVector2D Icon64 = FVector2D(64.0f, 64.0f);

				// WaveTabe Editor
				{
					// Class Icons
					auto SetClassIcon = [this, InIcon16 = Icon16, InIcon64 = Icon64](const FString& ClassName)
					{
						const FString IconFileName = FString::Printf(TEXT("Icons/%s"), *ClassName.ToLower());
						const FSlateColor DefaultForeground(FStyleColors::Foreground);

						Set(*FString::Printf(TEXT("ClassIcon.%s"), *ClassName), new IMAGE_BRUSH_SVG(IconFileName, InIcon16));
						Set(*FString::Printf(TEXT("ClassThumbnail.%s"), *ClassName), new IMAGE_BRUSH_SVG(IconFileName, InIcon64));
					};

					SetClassIcon(TEXT("WaveTableBank"));
				}

				FSlateStyleRegistry::RegisterSlateStyle(*this);
			}
		};
	} // namespace ModulePrivate

	void FModule::StartupModule()
	{
		using namespace ModulePrivate;

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorName);
		PropertyModule.RegisterCustomPropertyTypeLayout
		(
			"WaveTableTransform",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTransformLayoutCustomization::MakeInstance)
		);

		PropertyModule.RegisterCustomPropertyTypeLayout
		(
			"WaveTableData",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWaveTableDataLayoutCustomization::MakeInstance)
		);

		ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>(CurveEditorName);
		FWaveTableCurveModel::WaveTableViewId = CurveEditorModule.RegisterView(FOnCreateCurveEditorView::CreateStatic(
			[](TWeakPtr<FCurveEditor> WeakCurveEditor) -> TSharedRef<SCurveEditorView>
			{
				return SNew(SViewStacked, WeakCurveEditor);
			}
		));

		StyleSet = MakeShared<FSlateStyle>();
	}

	void FModule::ShutdownModule()
	{
		using namespace ModulePrivate;

		if (FModuleManager::Get().IsModuleLoaded(PropertyEditorName))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorName);
			PropertyModule.UnregisterCustomPropertyTypeLayout("WaveTableTransform");
		}

		if (FModuleManager::Get().IsModuleLoaded(CurveEditorName))
		{
			ICurveEditorModule& CurveEditorModule = FModuleManager::LoadModuleChecked<ICurveEditorModule>(CurveEditorName);
			CurveEditorModule.UnregisterView(FWaveTableCurveModel::WaveTableViewId);
			FWaveTableCurveModel::WaveTableViewId = ECurveEditorViewID::Invalid;
		}
	}
} // namespace WaveTable::Editor

IMPLEMENT_MODULE(WaveTable::Editor::FModule, WaveTableEditor);
