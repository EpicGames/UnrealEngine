// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetTypeActions_Base.h"
#include "Modules/ModuleInterface.h"
#include "Styling/SlateStyle.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogWaveTableEditor, Log, All);


namespace WaveTable::Editor
{
	class FModule : public IModuleInterface
	{
	public:
		FModule() = default;

		virtual void StartupModule() override;

		virtual void ShutdownModule() override;

	private:
		TArray<TSharedPtr<FAssetTypeActions_Base>> AssetActions;
		TSharedPtr<FSlateStyleSet> StyleSet;
	};
} // namespace WaveTable::Editor
