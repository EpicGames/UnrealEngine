// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableFileUtilities.h"

#include "Factories/SoundFactory.h"
#include "Sound/SoundWave.h"
#include "UObject/Package.h"
#include "Utils.h"
#include "WaveTableEditorModule.h"
#include "WaveTableSampler.h"


namespace WaveTable::Editor
{
	namespace FileUtilities
	{
		void LoadPCMChannel(const FString& InFilePath, int32 InChannelIndex, FWaveTableData& OutData, int32& OutSampleRate)
		{
			OutData.Empty();

			if (!InFilePath.IsEmpty())
			{
				if (USoundFactory* SoundWaveFactory = NewObject<USoundFactory>())
				{
					// Setup sane defaults for importing localized sound waves
					SoundWaveFactory->bAutoCreateCue = false;
					SoundWaveFactory->SuppressImportDialogs();

					UPackage* TempPackage = CreatePackage(TEXT("/Temp/WaveTable"));
					if (USoundWave* SoundWave = ImportObject<USoundWave>(TempPackage, "ImportedWaveTable", RF_Public | RF_Standalone, *InFilePath, nullptr, SoundWaveFactory))
					{
						TArray<uint8> RawPCMData;
						uint16 NumChannels = 0;
						uint32 FileSampleRate = 0;
						SoundWave->GetImportedSoundWaveData(RawPCMData, FileSampleRate, NumChannels);
						OutSampleRate = static_cast<int32>(FileSampleRate);

						// Deinterleave & perform BDC if necessary
						if (NumChannels > 0)
						{
							const int32 NumFrames = RawPCMData.Num() / NumChannels / sizeof(int16);
							const int16* RawDataPtr = (const int16*)(RawPCMData.GetData());
							const int32 ChannelOffset = (InChannelIndex % NumChannels);

							OutData.Zero(NumFrames);

							constexpr float ConversionValue = 1.0f / static_cast<float>(TNumericLimits<int16>::Max());
							switch (OutData.GetBitDepth())
							{
								case EWaveTableBitDepth::IEEE_Float:
								{
									TArrayView<float> DataView;
									OutData.GetDataView(DataView);
									for (int32 Index = 0; Index < DataView.Num(); ++Index)
									{
										DataView[Index] = (RawDataPtr[(Index * NumChannels) + ChannelOffset]) * ConversionValue;
									}

									if (!DataView.IsEmpty())
									{
										OutData.SetFinalValue(DataView.Last());
									}
								}
								break;

								case EWaveTableBitDepth::PCM_16:
								{
									TArrayView<int16> DataView;
									OutData.GetDataView(DataView);
									for (int32 Index = 0; Index < DataView.Num(); ++Index)
									{
										DataView[Index] = (RawDataPtr[(Index * NumChannels) + ChannelOffset]);
									}

									if (!DataView.IsEmpty())
									{
										OutData.SetFinalValue(DataView.Last() * ConversionValue);
									}
								}
								break;

								default:
								{
									static_assert(static_cast<int32>(EWaveTableBitDepth::COUNT) == 2, "Possible missing switch case coverage for 'EWaveTableBitDepth'");
									checkNoEntry();
								}
								break;
							}
						}
						else
						{
							UE_LOGF(LogWaveTableEditor, Error, "Failed to import/reimport file '%ls': Invalid num channels '%u' imported", *InFilePath, NumChannels);
						}
					}
					else
					{
						UE_LOGF(LogWaveTableEditor, Error, "Failed to import/reimport file '%ls'", *InFilePath);
					}
				}
			}
		}

		void LoadPCMChannel(const FString& InFilePath, int32 InChannelIndex, TArray<float>& OutPCMData)
		{
			OutPCMData.Empty();

			if (!InFilePath.IsEmpty())
			{
				if (USoundFactory* SoundWaveFactory = NewObject<USoundFactory>())
				{
					// Setup sane defaults for importing localized sound waves
					SoundWaveFactory->bAutoCreateCue = false;
					SoundWaveFactory->SuppressImportDialogs();

					UPackage* TempPackage = CreatePackage(TEXT("/Temp/WaveTable"));
					if (USoundWave* SoundWave = ImportObject<USoundWave>(TempPackage, "ImportedWaveTable", RF_Public | RF_Standalone, *InFilePath, nullptr, SoundWaveFactory))
					{
						TArray<uint8> RawPCMData;
						uint32 SampleRate = 0;
						uint16 NumChannels = 0;
						SoundWave->GetImportedSoundWaveData(RawPCMData, SampleRate, NumChannels);

						if (NumChannels > 0)
						{
							const int32 NumFrames = RawPCMData.Num() / NumChannels / sizeof(int16);
							OutPCMData.Empty();
							OutPCMData.AddZeroed(NumFrames);
							const int16* RawDataPtr = (const int16*)(RawPCMData.GetData());
							const int32 ChannelOffset = (InChannelIndex % NumChannels);
							constexpr float Max16BitAsFloat = static_cast<float>(TNumericLimits<int16>::Max());
							for (int32 i = 0; i < OutPCMData.Num(); ++i)
							{
								OutPCMData[i] = (RawDataPtr[(i * NumChannels) + ChannelOffset]) / Max16BitAsFloat;
							}
						}
					}
				}
			}
		}
	} // namespace FileUtilities
} // namespace WaveTable::Editor
