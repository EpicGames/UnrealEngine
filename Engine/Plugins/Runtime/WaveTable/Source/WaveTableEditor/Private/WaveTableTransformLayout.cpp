// Copyright Epic Games, Inc. All Rights Reserved.
#include "WaveTableTransformLayout.h"

#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "WaveTable.h"
#include "WaveTableBank.h"
#include "WaveTableFileUtilities.h"
#include "WaveTableSettings.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "WaveTableEditor"


namespace WaveTable::Editor
{
	void FWaveTableDataLayoutCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
	}

	void FWaveTableDataLayoutCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		TSharedRef<IPropertyHandle> BitDepthHandle = PropertyHandles.FindChecked(FWaveTableData::GetBitDepthPropertyName()).ToSharedRef();
		ChildBuilder.AddProperty(BitDepthHandle);
	}

	void FTransformLayoutCustomizationBase::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
	}

	void FTransformLayoutCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		CurveHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FWaveTableTransform, Curve)).ToSharedRef();
		CustomizeCurveSelector(ChildBuilder);

		TSharedRef<IPropertyHandle> ScalarHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FWaveTableTransform, Scalar)).ToSharedRef();

		ChildBuilder.AddProperty(ScalarHandle)
			.EditCondition(TAttribute<bool>::Create([this]() { return IsScaleableCurve(); }), nullptr)
			.Visibility(TAttribute<EVisibility>::Create([this]() { return IsScaleableCurve() ? EVisibility::Visible : EVisibility::Hidden; }));

		TSharedRef<IPropertyHandle> SharedCurveHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FWaveTableTransform, CurveShared)).ToSharedRef();
		ChildBuilder.AddProperty(SharedCurveHandle)
			.EditCondition(TAttribute<bool>::Create([this]() { return GetCurve() == EWaveTableCurve::Shared; }), nullptr)
			.Visibility(TAttribute<EVisibility>::Create([this]() { return GetCurve() == EWaveTableCurve::Shared ? EVisibility::Visible : EVisibility::Hidden; }));

		TSharedRef<IPropertyHandle> DurationHandle = PropertyHandles.FindChecked(FWaveTableTransform::GetDurationPropertyName()).ToSharedRef();
		ChildBuilder.AddProperty(DurationHandle)
			.EditCondition(TAttribute<bool>::Create([this]()
			{
				const bool bIsSampleRateMode = GetSamplingMode() == EWaveTableSamplingMode::FixedSampleRate;
				const bool bIsNotFile = GetCurve() != EWaveTableCurve::File;
				return bIsSampleRateMode && bIsNotFile;
			}), nullptr)
			.Visibility(TAttribute<EVisibility>::Create([this]()
			{
				const bool bIsSampleRateMode = GetSamplingMode() == EWaveTableSamplingMode::FixedSampleRate;
				return bIsSampleRateMode ? EVisibility::Visible : EVisibility::Collapsed;
			}));

		TAttribute<EVisibility> IsWaveTableVisibilityAttribute = TAttribute<EVisibility>::Create([this]()
		{
			return GetCurve() == EWaveTableCurve::File ? EVisibility::Visible : EVisibility::Hidden;
		});

		WaveTableOptionsHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FWaveTableTransform, WaveTableSettings));

		uint32 NumWaveTableOptions = 0;
		if (WaveTableOptionsHandle->GetNumChildren(NumWaveTableOptions))
		{
			ChannelIndexHandle = WaveTableOptionsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FWaveTableSettings, ChannelIndex));
			ChannelIndexHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this] { CachePCMFromFile(); }));

			FilePathHandle = WaveTableOptionsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FWaveTableSettings, FilePath));
			FilePathHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([this] { CachePCMFromFile(); }));

			FilePathHandle = FilePathHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FFilePath, FilePath));

			SourceDataHandle = WaveTableOptionsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FWaveTableSettings, SourceData));
			SourceDataHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([this] { CachePCMFromFile(); }));
		}

		ChildBuilder.AddProperty(WaveTableOptionsHandle->AsShared())
			.EditCondition(TAttribute<bool>::Create([this]() { return GetCurve() == EWaveTableCurve::File; }), nullptr)
			.Visibility(IsWaveTableVisibilityAttribute);
	}

	void FTransformLayoutCustomizationBase::CustomizeCurveSelector(IDetailChildrenBuilder& ChildBuilder)
	{
		check(CurveHandle.IsValid());

		auto GetAll = [this, SupportedCurves = GetSupportedCurves()](TArray<TSharedPtr<FString>>& OutNames, TArray<TSharedPtr<SToolTip>>& OutTooltips, TArray<bool>&)
		{
			CurveDisplayStringToNameMap.Reset();
			UEnum* Enum = StaticEnum<EWaveTableCurve>();
			check(Enum);

			static const int64 MaxValue = static_cast<int64>(EWaveTableCurve::Count);
			for (int32 i = 0; i < Enum->NumEnums(); ++i)
			{
				EWaveTableCurve Curve = static_cast<EWaveTableCurve>(Enum->GetValueByIndex(i));
				if (SupportedCurves.Contains(Curve))
				{
					TSharedPtr<FString> DisplayString = MakeShared<FString>(Enum->GetDisplayNameTextByIndex(i).ToString());

					const FName Name = Enum->GetNameByIndex(i);
					CurveDisplayStringToNameMap.Add(*DisplayString.Get(), Name);

					OutTooltips.Emplace(SNew(SToolTip).Text(Enum->GetToolTipTextByIndex(i)));
					OutNames.Emplace(MoveTemp(DisplayString));
				}
			}
		};

		auto GetValue = [this]()
		{
			UEnum* Enum = StaticEnum<EWaveTableCurve>();
			check(Enum);

			uint8 IntValue;
			if (CurveHandle->GetValue(IntValue) == FPropertyAccess::Success)
			{
				return Enum->GetDisplayNameTextByValue(IntValue).ToString();
			}

			return Enum->GetDisplayNameTextByValue(static_cast<int32>(EWaveTableCurve::Count)).ToString();
		};

		auto SelectedValue = [this](const FString& InSelected)
		{
			UEnum* Enum = StaticEnum<EWaveTableCurve>();
			check(Enum);

			const FName& Name = CurveDisplayStringToNameMap.FindChecked(InSelected);
			const uint8 Value = static_cast<uint8>(Enum->GetValueByName(Name, EGetByNameFlags::ErrorIfNotFound));
			ensure(CurveHandle->SetValue(Value) == FPropertyAccess::Success);
		};

		static const FText CurveDisplayName = LOCTEXT("WaveTableTransformCurveProperty", "Curve");
		ChildBuilder.AddCustomRow(CurveDisplayName)
			.NameContent()
			[
				CurveHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(150.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
			[
				SNew(SBox)
				[
					PropertyCustomizationHelpers::MakePropertyComboBox(
						CurveHandle,
						FOnGetPropertyComboBoxStrings::CreateLambda(GetAll),
						FOnGetPropertyComboBoxValue::CreateLambda(GetValue),
						FOnPropertyComboBoxValueSelected::CreateLambda(SelectedValue)
					)
				]
			]
		];
	}

	EWaveTableSamplingMode FTransformLayoutCustomizationBase::GetSamplingMode() const
	{
		TArray<UObject*> OuterObjects;
		WaveTableOptionsHandle->GetOuterObjects(OuterObjects);
		for (UObject* Object : OuterObjects)
		{
			if (UWaveTableBank* Bank = Cast<UWaveTableBank>(Object))
			{
				return Bank->SampleMode;
			}
		}

		return EWaveTableSamplingMode::FixedResolution;
	}

	void FTransformLayoutCustomizationBase::CachePCMFromFile()
	{
		using namespace WaveTable::Editor;

		TArray<UObject*> OuterObjects;
		WaveTableOptionsHandle->GetOuterObjects(OuterObjects);
		for (UObject* Object : OuterObjects)
		{
			if (UWaveTableBank* Bank = Cast<UWaveTableBank>(Object))
			{
				if (FWaveTableTransform* Transform = GetTransform())
				{
					FWaveTableSettings& Settings = Transform->WaveTableSettings;
					const FString& FilePath = Settings.FilePath.FilePath;
					const int32 ChannelIndex = Settings.ChannelIndex;
					FWaveTableData& SourceData = Settings.SourceData;
					int32& SampleRate = Settings.SourceSampleRate;
					FileUtilities::LoadPCMChannel(FilePath, ChannelIndex, SourceData, SampleRate);
				}

				// TODO: Only refresh table associated with this transform
				Bank->RefreshWaveTables();
			}
		}
	}

	bool FTransformLayoutCustomizationBase::IsScaleableCurve() const
	{
		static const TArray<EWaveTableCurve> ScalarFilters = { EWaveTableCurve::Exp, EWaveTableCurve::Exp_Inverse, EWaveTableCurve::Log };
		return ScalarFilters.Contains(GetCurve());
	}

	EWaveTableCurve FTransformLayoutCustomizationBase::GetCurve() const
	{
		if (!CurveHandle.IsValid())
		{
			return EWaveTableCurve::Linear;
		}

		uint8 CurveValue = static_cast<uint8>(EWaveTableCurve::Linear);
		CurveHandle->GetValue(CurveValue);
		return static_cast<EWaveTableCurve>(CurveValue);
	}

	int32 FTransformLayoutCustomizationBase::GetOwningArrayIndex() const
	{
		int32 TableIndex = INDEX_NONE;
		if (ensure(WaveTableOptionsHandle.IsValid()))
		{
			TSharedPtr<IPropertyHandleArray> ParentArray;
			TSharedPtr<IPropertyHandle> ChildHandle = WaveTableOptionsHandle;
			while (ChildHandle.IsValid() && TableIndex == INDEX_NONE)
			{
				TableIndex = ChildHandle->GetIndexInArray();
				ChildHandle = ChildHandle->GetParentHandle();
			}
		}

		return TableIndex;
	}

	TSet<EWaveTableCurve> FTransformLayoutCustomizationBase::GetSupportedCurves() const
	{
		UEnum* Enum = StaticEnum<EWaveTableCurve>();
		check(Enum);

		TSet<EWaveTableCurve> Curves;
		static const int64 MaxValue = static_cast<int64>(EWaveTableCurve::Count);
		for (int32 i = 0; i < Enum->NumEnums(); ++i)
		{
			if (Enum->GetValueByIndex(i) < MaxValue)
			{
				if (!Enum->HasMetaData(TEXT("Hidden"), i))
				{
					Curves.Add(static_cast<EWaveTableCurve>(Enum->GetValueByIndex(i)));
				}
			}
		}

		return Curves;
	}

	bool FTransformLayoutCustomization::IsBipolar() const
	{
		if (ensure(WaveTableOptionsHandle.IsValid()))
		{
			TArray<UObject*> OuterObjects;
			WaveTableOptionsHandle->GetOuterObjects(OuterObjects);
			if (OuterObjects.Num() == 1)
			{
				if (UWaveTableBank* Bank = Cast<UWaveTableBank>(OuterObjects.Last()))
				{
					return Bank->bBipolar;
				}
			}
		}

		return false;
	}

	FWaveTableTransform* FTransformLayoutCustomization::GetTransform() const
	{
		if (ensure(WaveTableOptionsHandle.IsValid()))
		{
			TArray<UObject*> OuterObjects;
			WaveTableOptionsHandle->GetOuterObjects(OuterObjects);
			if (OuterObjects.Num() == 1)
			{
				if (UWaveTableBank* Bank = Cast<UWaveTableBank>(OuterObjects.Last()))
				{
					const int32 TableIndex = GetOwningArrayIndex();
					if (TableIndex != INDEX_NONE)
					{
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						if (TableIndex < Bank->Entries.Num())
						{
							return &Bank->Entries[TableIndex].Transform;
						}
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}
			}
		}

		return nullptr;
	}
} // namespace WaveTable::Editor
#undef LOCTEXT_NAMESPACE // WaveTableEditor
