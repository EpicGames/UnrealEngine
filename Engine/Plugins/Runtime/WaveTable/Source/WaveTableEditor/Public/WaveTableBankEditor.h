// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WaveTableSettings.h"

#define UE_API WAVETABLEEDITOR_API

class FSpawnTabArgs;
enum class EWaveTableCurveSource : uint8;
namespace WaveTable::Editor { class FWaveTableCurveModel; }
struct FRichCurve;


// Forward Declarations
class FCurveEditor;
struct FWaveTableTransform;
class IToolkitHost;
class SCurveEditorPanel;
class UCurveFloat;

namespace WaveTable
{
	namespace Editor
	{
		class FBankEditorBase : public FAssetEditorToolkit, public FNotifyHook, public FEditorUndoClient
		{
		public:
			UE_API FBankEditorBase();
			virtual ~FBankEditorBase() = default;

			UE_API void Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* InParentObject);

			/** FAssetEditorToolkit interface */
			UE_API virtual FName GetToolkitFName() const override;
			UE_API virtual FText GetBaseToolkitName() const override;
			UE_API virtual FString GetWorldCentricTabPrefix() const override;
			UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
			UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
			UE_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

			/** FNotifyHook interface */
			UE_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

			// Regenerates curves set to "file" without refreshing whole stack view
			UE_API void RegenerateFileCurves();

			/** Updates & redraws curves. */
			UE_API void RefreshCurves();

		protected:
			struct FCurveData
			{
				FCurveModelID ModelID;

				/* Curve used purely for display.  May be down-sampled from
				 * asset's curve for performance while editing */
				TSharedPtr<FRichCurve> ExpressionCurve;

				FCurveData()
					: ModelID(FCurveModelID::Unique())
				{
				}
			};

			UE_API virtual void PostUndo(bool bSuccess) override;
			UE_API virtual void PostRedo(bool bSuccess) override;

			virtual bool GetIsPropertyEditorDisabled() const { return false; }

			// Returns resolution of bank being edited. By default, bank does not support WaveTable generation
			// by being set to no resolution.
			virtual EWaveTableResolution GetBankResolution() const { return EWaveTableResolution::None; }

			// Returns sampling mode of bank being edited.
			virtual EWaveTableSamplingMode GetBankSamplingMode() const { return EWaveTableSamplingMode::FixedResolution; }

			// Returns sample rate to use if BankSamplingMode is set to 'SampleRate'.
			virtual int32 GetBankSampleRate() const { return 0; }

			// Returns whether or not bank is bipolar.  By default, returns false (bank is unipolar), functionally
			// operating as a unipolar envelope editor.
			virtual bool GetBankIsBipolar() const { return false; }

			// Construct a new curve model for the given FRichCurve.  Allows for editor implementation to construct custom curve model types.
			virtual TUniquePtr<FWaveTableCurveModel> ConstructCurveModel(FRichCurve& InRichCurve, UObject* InParentObject, EWaveTableCurveSource InSource) = 0;

			// Returns the transform associated with the given index
			virtual FWaveTableTransform* GetTransform (int32 InIndex) const = 0;

			// Returns the number of transforms associated with the given bank.
			virtual int32 GetNumTransforms() const = 0;

			UE_API void SetCurve(int32 InTransformIndex, FRichCurve& InRichCurve, EWaveTableCurveSource InSource);

		private:
			// Regenerates expression curve at the given index. If curve is a 'File' and source PCM data in Transform
			// is too long, inline edits are not included in recalculation for better interact performance.
			UE_API void GenerateExpressionCurve(FCurveData& OutCurveData, int32 InTransformIndex, bool bInIsUnset = false);

			UE_API void InitCurves();

			UE_API void ResetCurves();

			/**	Spawns the tab allowing for editing/viewing the output curve(s) */
			UE_API TSharedRef<SDockTab> SpawnTab_OutputCurve(const FSpawnTabArgs& Args);

			/**	Spawns the tab allowing for editing/viewing the output curve(s) */
			UE_API TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

			/** Get the orientation for the snap value controls. */
			UE_API EOrientation GetSnapLabelOrientation() const;

			/** Trims keys out-of-bounds in provided curve */
			static UE_API void TrimKeys(FRichCurve& OutCurve);

			/** Clears the expression curve at the given input index */
			UE_API void ClearExpressionCurve(int32 InTransformIndex);

			UE_API bool RequiresNewCurve(int32 InTransformIndex, const FRichCurve& InRichCurve) const;

			TSharedPtr<FUICommandList> ToolbarCurveTargetCommands;

			TSharedPtr<FCurveEditor> CurveEditor;
			TSharedPtr<SCurveEditorPanel> CurvePanel;

			TArray<FCurveData> CurveData;

			/** Properties tab */
			TSharedPtr<IDetailsView> PropertiesView;

			/** Settings Editor App Identifier */
			static UE_API const FName AppIdentifier;
			static UE_API const FName CurveTabId;
			static UE_API const FName PropertiesTabId;
		};

		class FBankEditor : public FBankEditorBase
		{
		public:
			FBankEditor() = default;
			virtual ~FBankEditor() = default;

		protected:
			UE_API virtual EWaveTableResolution GetBankResolution() const override;
			UE_API virtual EWaveTableSamplingMode GetBankSamplingMode() const override;
			UE_API virtual int32 GetBankSampleRate() const override;
			UE_API virtual bool GetBankIsBipolar() const override;
			UE_API virtual int32 GetNumTransforms() const override;
			UE_API virtual FWaveTableTransform* GetTransform(int32 InIndex) const override;

			UE_API virtual TUniquePtr<FWaveTableCurveModel> ConstructCurveModel(FRichCurve& InRichCurve, UObject* InParentObject, EWaveTableCurveSource InSource) override;
		};
	} // namespace Editor
} // namespace WaveTable

#undef UE_API
