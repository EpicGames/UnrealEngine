// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RichCurveEditorModel.h"
#include "Views/SCurveEditorViewStacked.h"
#include "WaveTableSettings.h"
#include "WaveTableTransformLayout.h"

#include "WaveTableCurveEditorViewStacked.generated.h"

#define UE_API WAVETABLEEDITOR_API


// Forward Declarations
class UCurveFloat;
struct FWaveTableTransform;
enum class ECurveEditorViewID : uint64;
enum class EWaveTableSamplingMode : uint8;


UENUM()
enum class EWaveTableCurveSource : uint8
{
	Custom,
	Expression,
	Shared,
	Unset
};


namespace WaveTable
{
	namespace Editor
	{
		class FWaveTableCurveModel : public FRichCurveEditorModelRaw
		{
		public:
			static UE_API ECurveEditorViewID WaveTableViewId;

			UE_API FWaveTableCurveModel(FRichCurve& InRichCurve, UObject* InOwner, EWaveTableCurveSource InSource);

			UE_API const FText& GetAxesDescriptor() const;
			UE_API const UObject* GetParentObject() const;
			UE_API EWaveTableCurveSource GetSource() const;
			UE_API void Refresh(const FWaveTableTransform& InTransform, int32 InCurveIndex, bool bInIsBipolar, EWaveTableSamplingMode InSamplingMode = EWaveTableSamplingMode::FixedResolution);

			virtual ECurveEditorViewID GetViewId() const { return WaveTableViewId; }
			UE_API virtual bool IsReadOnly() const override;
			UE_API virtual FLinearColor GetColor() const override;
			UE_API virtual void GetTimeRange(double& MinValue, double& MaxValue) const override;
			UE_API virtual void GetValueRange(double& MinValue, double& MaxValue) const override;

			UE_API virtual bool ShowDefaultDashedLine() const { return false; }
			UE_API virtual double GetDefaultDashedLineValue() const { return 1.0; } 

			float GetCurveDuration() const { return CurveDuration; }
			int32 GetCurveIndex() const { return CurveIndex; }
			bool GetIsBipolar() const { return bIsBipolar; }
			float GetFadeInRatio() const { return FadeInRatio; }
			float GetFadeOutRatio() const { return FadeOutRatio; }
			int32 GetNumSamples() const { return NumSamples; }
			EWaveTableSamplingMode GetSamplingMode() const { return SamplingMode; }
		protected:
			UE_API virtual void RefreshCurveDescriptorText(const FWaveTableTransform& InTransform, FText& OutShortDisplayName, FText& OutInputAxisName, FText& OutOutputAxisName);
			UE_API virtual FColor GetCurveColor() const;
			UE_API virtual bool GetPropertyEditorDisabled() const;
			UE_API virtual FText GetPropertyEditorDisabledText() const;

			TWeakObjectPtr<UObject> ParentObject;

		private:
			int32 CurveIndex = INDEX_NONE;
			int32 NumSamples = 0;

			EWaveTableCurveSource Source = EWaveTableCurveSource::Unset;
			EWaveTableSamplingMode SamplingMode = EWaveTableSamplingMode::FixedResolution;

			bool bIsBipolar = false;

			float FadeInRatio = 0.0f;
			float FadeOutRatio = 0.0f;
			float CurveDuration = 1.0f;

			FText InputAxisName;
			FText AxesDescriptor;
		};

		class SViewStacked : public SCurveEditorViewStacked
		{
		public:
			UE_API void Construct(const FArguments& InArgs, TWeakPtr<FCurveEditor> InCurveEditor);

		protected:
			UE_API virtual void PaintView(
				const FPaintArgs& Args,
				const FGeometry& AllottedGeometry,
				const FSlateRect& MyCullingRect,
				FSlateWindowElementList& OutDrawElements,
				int32 BaseLayerId,
				const FWidgetStyle& InWidgetStyle,
				bool bParentEnabled) const override;

			UE_API virtual void DrawViewGrids(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const override;
			UE_API virtual void DrawLabels(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 BaseLayerId, ESlateDrawEffect DrawEffects) const override;

			UE_API virtual void FormatInputLabel(const FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const;
			virtual void FormatOutputLabel(const FWaveTableCurveModel& EditorModel, const FNumberFormattingOptions& InLabelFormat, FText& InOutLabel) const { }
			UE_API virtual FText FormatToolTipTime(const FCurveModel& CurveModel, double EvaluatedTime) const override;

		private:
			struct FGridDrawInfo
			{
				const FGeometry* AllottedGeometry = nullptr;

				FPaintGeometry PaintGeometry;

				ESlateDrawEffect DrawEffects;
				FNumberFormattingOptions LabelFormat;

				TArray<FVector2D> LinePoints;

				FCurveEditorScreenSpace ScreenSpace;

			private:
				FLinearColor MajorGridColor;
				FLinearColor MinorGridColor;

				int32 BaseLayerId = INDEX_NONE;

				const FCurveModel* CurveModel = nullptr;

				double LowerValue = 0.0;
				double PixelBottom = 0.0;
				double PixelTop = 0.0;

			public:
				FGridDrawInfo(const FGeometry* InAllottedGeometry, const FCurveEditorScreenSpace& InScreenSpace, FLinearColor InGridColor, int32 InBaseLayerId);

				void SetCurveModel(const FCurveModel* InCurveModel);
				const FCurveModel* GetCurveModel() const;
				void SetLowerValue(double InLowerValue);
				int32 GetBaseLayerId() const;
				FLinearColor GetLabelColor() const;
				double GetLowerValue() const;
				FLinearColor GetMajorGridColor() const;
				FLinearColor GetMinorGridColor() const;
				double GetPixelBottom() const;
				double GetPixelTop() const;
			};

			UE_API void DrawViewGridLineX(FSlateWindowElementList& OutDrawElements, FGridDrawInfo& DrawInfo, ESlateDrawEffect DrawEffect, double OffsetAlpha, bool bIsMajor, bool bDashedLine = false) const;
			UE_API void DrawViewGridLineY(const float VerticalLine, FSlateWindowElementList& OutDrawElements, FGridDrawInfo &DrawInfo, ESlateDrawEffect DrawEffects, const FText* Label, bool bIsMajor) const;
		};
	} // namespace Editor
} // namespace WaveTable

#undef UE_API
