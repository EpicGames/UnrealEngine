// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "WaveTable.h"


namespace WaveTable::Editor
{
	namespace FileUtilities
	{
		void WAVETABLEEDITOR_API LoadPCMChannel(const FString& InFilePath, int32 InChannelIndex, FWaveTableData& OutData, int32& OutSampleRate);

		UE_DEPRECATED(5.3, "Use version of 'LoadPCMChannel' that takes in WaveTableData to support sample rate & bit depth conversion")
		void WAVETABLEEDITOR_API LoadPCMChannel(const FString& InFilePath, int32 InChannelIndex, TArray<float>& OutPCMData);
	} // namespace FileUtilities
} // namespace WaveTable::Editor
