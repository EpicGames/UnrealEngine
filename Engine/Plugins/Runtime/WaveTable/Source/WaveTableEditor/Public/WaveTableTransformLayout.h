// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"

#define UE_API WAVETABLEEDITOR_API


// Forward Declarations
enum class EWaveTableCurve : uint8;
enum class EWaveTableSamplingMode : uint8;
struct FWaveTableTransform;


namespace WaveTable::Editor
{
	class FWaveTableDataLayoutCustomization : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance()
		{
			return MakeShared<FWaveTableDataLayoutCustomization>();
		}

		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	};

	class FTransformLayoutCustomizationBase : public IPropertyTypeCustomization
	{
	public:
		//~ Begin IPropertyTypeCustomization
		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization

	protected:
		UE_API virtual TSet<EWaveTableCurve> GetSupportedCurves() const;
		virtual FWaveTableTransform* GetTransform() const = 0;
		virtual bool IsBipolar() const = 0;
		UE_API virtual EWaveTableSamplingMode GetSamplingMode() const;

		UE_API void CachePCMFromFile();
		UE_API EWaveTableCurve GetCurve() const;
		UE_API int32 GetOwningArrayIndex() const;
		UE_API bool IsScaleableCurve() const;

		TSharedPtr<IPropertyHandle> CurveHandle;
		TSharedPtr<IPropertyHandle> ChannelIndexHandle;
		TSharedPtr<IPropertyHandle> FilePathHandle;
		TSharedPtr<IPropertyHandle> SourceDataHandle;
		TSharedPtr<IPropertyHandle> WaveTableOptionsHandle;

	private:
		UE_API void CustomizeCurveSelector(IDetailChildrenBuilder& ChildBuilder);

		TMap<FString, FName> CurveDisplayStringToNameMap;
	};

	class FTransformLayoutCustomization : public FTransformLayoutCustomizationBase
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance()
		{
			return MakeShared<FTransformLayoutCustomization>();
		}

		UE_API virtual bool IsBipolar() const override;
		UE_API virtual FWaveTableTransform* GetTransform() const override;
	};
} // namespace WaveTable::Editor

#undef UE_API
