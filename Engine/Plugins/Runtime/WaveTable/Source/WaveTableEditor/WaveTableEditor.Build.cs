// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WaveTableEditor : ModuleRules
{
	public WaveTableEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"AssetTools",
			"SlateCore",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetDefinition",
			"AudioEditor",
			"AudioExtensions",
			"Core",
			"CoreUObject",
			"CurveEditor",
			"Engine",
			"EditorFramework",
			"EditorWidgets",
			"GameProjectGeneration",
			"InputCore",
			"PropertyEditor",
			"SequenceRecorder",
			"Slate",
			"ToolWidgets",
			"UnrealEd",
			"WaveTable",
		});
	}
}
